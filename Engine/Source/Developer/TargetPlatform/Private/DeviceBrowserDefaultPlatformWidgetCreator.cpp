// Copyright Epic Games, Inc. All Rights Reserved.

#include "DeviceBrowserDefaultPlatformWidgetCreator.h"
#include "Internationalization/Text.h"
#include "Misc/MessageDialog.h"
#include "Misc/CoreMisc.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "PlatformInfo.h"
#include "SDeviceBrowserDefaultPlatformAddWidget.h"

#define LOCTEXT_NAMESPACE "FDeviceBrowserDefaultPlatformWidgetCreator"

bool FDeviceBrowserDefaultPlatformWidgetCreator::IsAddDeviceInputValid(const FString& InPlatformName, const TSharedPtr<SWidget>& UserData)
{
	SDeviceBrowserDefaultPlatformAddWidget& CustomWidget = static_cast<SDeviceBrowserDefaultPlatformAddWidget&>(*UserData);

	ITargetPlatform* Platform = GetTargetPlatformManager()->FindTargetPlatform(*InPlatformName);
	check(Platform);

	FString TextCheck = CustomWidget.DeviceIdTextBox->GetText().ToString();
	TextCheck.TrimStartAndEndInline();

	if (!TextCheck.IsEmpty())
	{
		if (Platform->RequiresUserCredentials() != EPlatformAuthentication::Always)
		{
			return true;
		}

		// check user/password as well
		TextCheck = CustomWidget.UserNameTextBox->GetText().ToString();
		TextCheck.TrimStartAndEndInline();

		if (!TextCheck.IsEmpty())
		{
			// do not trim the password
			return !CustomWidget.UserPasswordTextBox->GetText().ToString().IsEmpty();
		}
	}

	return false;
}

void FDeviceBrowserDefaultPlatformWidgetCreator::AddDevice(const FString& InPlatformName, const TSharedPtr<SWidget>& UserData)
{
	SDeviceBrowserDefaultPlatformAddWidget& CustomWidget = static_cast<SDeviceBrowserDefaultPlatformAddWidget&>(*UserData);

	ITargetPlatform* Platform = GetTargetPlatformManager()->FindTargetPlatform(*InPlatformName);
	check(Platform);

	FString DeviceIdString = CustomWidget.DeviceIdTextBox->GetText().ToString();
	FString UserNameString = CustomWidget.UserNameTextBox->GetText().ToString();
	FString UserPassString = CustomWidget.UserPasswordTextBox->GetText().ToString();
	FString DeviceUserFriendlyNameString = CustomWidget.DeviceNameTextBox->GetText().ToString();
	bool bAdded = Platform->AddDevice(DeviceIdString, DeviceUserFriendlyNameString, UserNameString, UserPassString, false);
	if (bAdded)
	{
		CustomWidget.DeviceIdTextBox->SetText(FText::GetEmpty());
		CustomWidget.DeviceNameTextBox->SetText(FText::GetEmpty());
		CustomWidget.UserNameTextBox->SetText(FText::GetEmpty());
		CustomWidget.UserPasswordTextBox->SetText(FText::GetEmpty());
	}
	else
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("DeviceAdderFailedToAddDeviceMessage", "Failed to add the device!"));
	}
}

TSharedPtr<SWidget> FDeviceBrowserDefaultPlatformWidgetCreator::CreateAddDeviceWidget(const FString& InPlatformName)
{
	return SNew(SDeviceBrowserDefaultPlatformAddWidget, InPlatformName);
}

TSharedPtr<SWidget> FDeviceBrowserDefaultPlatformWidgetCreator::CreateDeviceInfoWidget(const FString& InPlatformName, const ITargetDevicePtr& InDevice)
{
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
