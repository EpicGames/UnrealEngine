// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Interfaces/IDeviceManagerCustomPlatformWidgetCreator.h"

class FDeviceBrowserDefaultPlatformWidgetCreator : public IDeviceManagerCustomPlatformWidgetCreator
{
public:
	virtual TSharedPtr<SWidget> CreateAddDeviceWidget(const FString& InPlatformName) override;
	virtual bool IsAddDeviceInputValid(const FString& InPlatformName, const TSharedPtr<SWidget>& UserData) override;
	
	virtual TSharedPtr<SWidget> CreateDeviceInfoWidget(const FString& InPlatformName, const ITargetDevicePtr& InDevice) override;

	virtual void AddDevice(const FString& InPlatformName, const TSharedPtr<SWidget>& UserData) override;
};
