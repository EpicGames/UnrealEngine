// Copyright Epic Games, Inc. All Rights Reserved.

#include "SDeviceBrowserDefaultPlatformAddWidget.h"

#include "Internationalization/Text.h"
#include "Misc/MessageDialog.h"
#include "Misc/CoreMisc.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "PlatformInfo.h"

#define LOCTEXT_NAMESPACE "SDeviceBrowserDefaultPlatformAddWidget"

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SDeviceBrowserDefaultPlatformAddWidget::Construct(const FArguments& InArgs, const FString& InPlatformName)
{
	// callback for determining the visibility of the credentials box
	auto CredentialsBoxVisibility = [this, InPlatformName]() -> EVisibility {
		ITargetPlatform* Platform = GetTargetPlatformManager()->FindTargetPlatform(*InPlatformName);

		if ((Platform != nullptr) && Platform->RequiresUserCredentials() != EPlatformAuthentication::Never)
		{
			return EVisibility::Visible;
		}

		return EVisibility::Collapsed;
	};

	// construct children
	ChildSlot
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Fill)
		  [
				SNew(SHorizontalBox)

				// device identifier input
				+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SVerticalBox)
						.ToolTipText(LOCTEXT("DeviceIdToolTip", "The device's unique identifier. Depending on the selected Platform, this can be a host name, an IP address, a MAC address or some other platform specific unique identifier."))

						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Left)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("DeviceIdLabel", "Device Identifier:"))
						]

					+ SVerticalBox::Slot()
						.FillHeight(1.0)
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							SAssignNew(DeviceIdTextBox, SEditableTextBox)
						]
					]

				// device name input
				+ SHorizontalBox::Slot()
					.HAlign(HAlign_Fill)
					.Padding(4.0f, 0.0f, 0.0f, 0.0f)
					[
						SNew(SVerticalBox)
						.ToolTipText(LOCTEXT("DeviceNameToolTip", "A display name for this device. Once the device is connected, this will be replaced with the device's actual name."))

						+ SVerticalBox::Slot()
						.AutoHeight()
						.HAlign(HAlign_Left)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("DisplayNameLabel", "Display Name:"))
						]

					+ SVerticalBox::Slot()
						.FillHeight(1.0)
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							SAssignNew(DeviceNameTextBox, SEditableTextBox)
						]
					]
			]

			+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				[
					SNew(SHorizontalBox)
						.Visibility_Lambda(CredentialsBoxVisibility)

					// user name input
					+ SHorizontalBox::Slot()
						.FillWidth(0.5f)
						.Padding(4.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(SVerticalBox)

							+ SVerticalBox::Slot()
								.AutoHeight()
								.HAlign(HAlign_Left)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("UserNameLabel", "User:"))
								]

							+ SVerticalBox::Slot()
								.FillHeight(1.0f)
								.Padding(0.0f, 4.0f, 0.0f, 0.0f)
								[
									SAssignNew(UserNameTextBox, SEditableTextBox)
								]
						]

					// user password input
					+ SHorizontalBox::Slot()
						.FillWidth(0.5f)
						.Padding(4.0f, 0.0f, 0.0f, 0.0f)
						[
							SNew(SVerticalBox)

							+ SVerticalBox::Slot()
								.AutoHeight()
								.HAlign(HAlign_Left)
								[
									SNew(STextBlock)
									.Text(LOCTEXT("UserPasswordLabel", "Password:"))
								]

					+ SVerticalBox::Slot()
						.FillHeight(1.0f)
						.Padding(0.0f, 4.0f, 0.0f, 0.0f)
						[
							SAssignNew(UserPasswordTextBox, SEditableTextBox)
							.IsPassword(true)
						]
						]
				]
		];
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

bool SDeviceBrowserDefaultPlatformAddWidget::IsInputValid(const FString& InPlatformName)
{
	FString TextCheck = DeviceNameTextBox->GetText().ToString();
	TextCheck.TrimStartAndEndInline();

	if (!TextCheck.IsEmpty())
	{
		ITargetPlatform* Platform = GetTargetPlatformManager()->FindTargetPlatform(*InPlatformName);

		if (!Platform || Platform->RequiresUserCredentials() != EPlatformAuthentication::Always)
		{
			return true;
		}

		// check user/password as well
		TextCheck = UserNameTextBox->GetText().ToString();
		TextCheck.TrimStartAndEndInline();

		if (!TextCheck.IsEmpty())
		{
			// do not trim the password
			return !UserPasswordTextBox->GetText().ToString().IsEmpty();
		}
	}

	return false;
}

#undef LOCTEXT_NAMESPACE
