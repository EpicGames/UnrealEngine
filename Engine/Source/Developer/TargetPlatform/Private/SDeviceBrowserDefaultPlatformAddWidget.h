// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SComboBox.h"

class FString;
class ITargetDeviceServiceManager;
class SButton;
class SEditableTextBox;
class SOverlay;

class SDeviceBrowserDefaultPlatformAddWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SDeviceBrowserDefaultPlatformAddWidget) { }
	SLATE_END_ARGS()

public:

	/**
	 * Construct the widget.
	 *
	 * @param InArgs The construction arguments.
	 * @param InDeviceManager The target device manager to use.
	 */
	void Construct(const FArguments& InArgs, const FString& InPlatformName);

	/** Checks if input user provided in all fields is valid */
	bool IsInputValid(const FString& InPlatformName);

	/** The device identifier text box.  */
	TSharedPtr<SEditableTextBox> DeviceIdTextBox;

	/** The device name text box. */
	TSharedPtr<SEditableTextBox> DeviceNameTextBox;

	/** The user name text box. */
	TSharedPtr<SEditableTextBox> UserNameTextBox;

	/** The user password text box. */
	TSharedPtr<SEditableTextBox> UserPasswordTextBox;

	/** The turnable overlay with user data. */
	TSharedPtr<SOverlay> UserDataOverlay;
};
