// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/TargetPlatformBase.h"
#include "Common/TargetPlatformControlsBase.h"
#include "HAL/IConsoleManager.h"
#include "DeviceBrowserDefaultPlatformWidgetCreator.h"
#include "Interfaces/IProjectBuildMutatorFeature.h"
#include "Features/IModularFeatures.h"
#include "Interfaces/IProjectManager.h"
#include "Interfaces/IPluginManager.h"
#include "ProjectDescriptor.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Sound/AudioFormatSettings.h"
#include "AnalyticsEventAttribute.h"

#define LOCTEXT_NAMESPACE "TargetPlatform"

PRAGMA_DISABLE_DEPRECATION_WARNINGS

bool FTargetPlatformBase::UsesForwardShading() const
{
	static IConsoleVariable* CVarForwardShading = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ForwardShading"));
	return CVarForwardShading ? (CVarForwardShading->GetInt() != 0) : false;
}

bool FTargetPlatformBase::UsesDBuffer() const
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DBuffer"));
	return CVar ? (CVar->GetInt() != 0) : false;
}

bool FTargetPlatformBase::UsesBasePassVelocity() const
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VelocityOutputPass"));
	return CVar ? (CVar->GetInt() == 1) : false;
}

bool FTargetPlatformBase::VelocityEncodeDepth() const
{
	return true;
}

bool FTargetPlatformBase::UsesSelectiveBasePassOutputs() const
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SelectiveBasePassOutputs"));
	return CVar ? (CVar->GetInt() != 0) : false;
}

bool FTargetPlatformBase::ShouldStripNaniteFallbackMeshes() const
{
	return false;
}

bool FTargetPlatformBase::UsesDistanceFields() const
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DistanceFields"));
	return CVar ? (CVar->GetInt() != 0) : false;
}

bool FTargetPlatformBase::UsesRayTracing() const
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RayTracing"));
	return CVar ? (CVar->GetInt() != 0) : false;
}

ERayTracingRuntimeMode FTargetPlatformBase::GetRayTracingMode() const
{
	return ERayTracingRuntimeMode::Disabled;
}

uint32 FTargetPlatformBase::GetSupportedHardwareMask() const
{
	return 0;
}

EOfflineBVHMode FTargetPlatformBase::GetStaticMeshOfflineBVHMode() const
{
	return EOfflineBVHMode::Disabled;
}

bool FTargetPlatformBase::GetStaticMeshOfflineBVHCompression() const
{
	return false;
}

EOfflineBVHMode FTargetPlatformBase::GetSkeletalMeshOfflineBVHMode() const
{
	return EOfflineBVHMode::Disabled;
}

bool FTargetPlatformBase::ForcesSimpleSkyDiffuse() const
{
	return false;
}

float FTargetPlatformBase::GetDownSampleMeshDistanceFieldDivider() const
{
	return 1.0f;
}

int32 FTargetPlatformBase::GetHeightFogModeForOpaque() const
{
	// Don't override the project setting by default
	// Platforms wish to support override need to implement the logic in their own target platform classes
	return 0;
}

bool FTargetPlatformBase::UsesMobileDBuffer() const
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DBuffer"));
	return CVar ? (CVar->GetInt() != 0) : false;
}

int32 GASTCHDRProfile = 0;
static FAutoConsoleVariableRef CVarAllowASTCHDRProfile(
	TEXT("cook.AllowASTCHDRProfile"),
	GASTCHDRProfile,
	TEXT("whether to allow ASTC HDR profile, the hdr format is only supported on some devices, e.g. Apple A13, Mali-G72, Adreno (TM) 660"),
	ECVF_Default | ECVF_ReadOnly
);

bool FTargetPlatformBase::UsesASTCHDR() const
{
	return (GASTCHDRProfile != 0);
}

FName FTargetPlatformBase::GetFallbackASTCHDR() const
{
	static IConsoleVariable* CVarFallbackForASTCHDRProfile = IConsoleManager::Get().FindConsoleVariable(TEXT("cook.FallbackForASTCHDRProfile"));
	static const FName NameRGBA16F(TEXT("RGBA16F"));
	static const FName NameASTC_RGB_LDR(TEXT("ASTC_RGB"));

	switch (CVarFallbackForASTCHDRProfile->GetInt())
	{
	case 0:
		return NameASTC_RGB_LDR;
	case 1:
		return NameRGBA16F;
	}
	return NameRGBA16F;
}

void FTargetPlatformBase::GetRayTracingShaderFormats(TArray<FName>& OutFormats) const
{
	if (UsesRayTracing())
	{
		GetAllTargetedShaderFormats(OutFormats);
	}
}

void FTargetPlatformBase::GetPlatformSpecificProjectAnalytics( TArray<FAnalyticsEventAttribute>& AnalyticsParamArray ) const
{
	AppendAnalyticsEventAttributeArray( AnalyticsParamArray,
		TEXT("UsesDistanceFields"), UsesDistanceFields(),
		TEXT("UsesForwardShading"), UsesForwardShading()
	);
}

void FTargetPlatformBase::AppendAnalyticsEventConfigBool( TArray<FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride )
{
	bool ConfigValue;
	if (GConfig->GetBool(ConfigSection, ConfigKey, ConfigValue, IniFileName))
	{
		AnalyticsParamArray.Add( FAnalyticsEventAttribute( AnalyticsKeyNameOverride ? AnalyticsKeyNameOverride : ConfigKey, ConfigValue ) );
	}
}

void FTargetPlatformBase::AppendAnalyticsEventConfigInt( TArray<FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride )
{
	int32 ConfigValue;
	if (GConfig->GetInt(ConfigSection, ConfigKey, ConfigValue, IniFileName))
	{
		AnalyticsParamArray.Add( FAnalyticsEventAttribute( AnalyticsKeyNameOverride ? AnalyticsKeyNameOverride : ConfigKey, ConfigValue ) );
	}
}

void FTargetPlatformBase::AppendAnalyticsEventConfigFloat( TArray<FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride )
{
	float ConfigValue;
	if (GConfig->GetFloat(ConfigSection, ConfigKey, ConfigValue, IniFileName))
	{
		AnalyticsParamArray.Add( FAnalyticsEventAttribute( AnalyticsKeyNameOverride ? AnalyticsKeyNameOverride : ConfigKey, ConfigValue ) );
	}
}

void FTargetPlatformBase::AppendAnalyticsEventConfigString( TArray<FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride )
{
	FString ConfigValue;
	if (GConfig->GetString(ConfigSection, ConfigKey, ConfigValue, IniFileName))
	{
		AnalyticsParamArray.Add( FAnalyticsEventAttribute( AnalyticsKeyNameOverride ? AnalyticsKeyNameOverride : ConfigKey, ConfigValue ) );
	}
}

void FTargetPlatformBase::AppendAnalyticsEventConfigArray( TArray<FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride )
{
	TArray<FString> ConfigValue;
	if (GConfig->GetArray(ConfigSection, ConfigKey, ConfigValue, IniFileName))
	{
		AnalyticsParamArray.Add( FAnalyticsEventAttribute( AnalyticsKeyNameOverride ? AnalyticsKeyNameOverride : ConfigKey, ConfigValue ) );
	}
}



static bool IsPluginEnabledForTarget(const IPlugin& Plugin, const FProjectDescriptor* Project, const FString& Platform, EBuildConfiguration Configuration, EBuildTargetType TargetType)
{
	if (!Plugin.GetDescriptor().SupportsTargetPlatform(Platform))
	{
		return false;
	}

	// TODO: Support transitive calculation of per-platform disabling for plugins.
	// Plugins can reference other plugins, and it would be nice to be able to automatically disable for platform X
	// plugins that are only referenced through another plugin that is disabled for platform X.
	// For the time-being, to disable a transitively referenced plugin per-platform, the project has to 
	// directly include the plugin.
	IPluginManager& PluginManager = IPluginManager::Get();

	if (Project != nullptr)
	{
		const FString& PluginName = Plugin.GetName();
		const FPluginReferenceDescriptor* PluginReference = Project->Plugins.FindByPredicate(
			[&PluginName](const FPluginReferenceDescriptor& ExistingReference)
			{
				return ExistingReference.Name == PluginName;
			});
		if (PluginReference)
		{
			// TODO: Remove this workaround for indirect plugin references. A project can mark a plugin as
			// "Enabled": false, but that merely prevents a direct reference, and the plugin might be referenced and
			// enabled by other plugins. PluginReference->IsEnabledForPlatform, IsEnabledForTargetConfiguration, and
			// IsEnabledForTarget will all return false in that case, even though the plugin is actually enabled.
			// Other systems using IPluginManager::Get().GetEnabledPlugins will disagree with the disabled result.
			// To workaround it, when we detect the case of a disabled plugin reference for a plugin that is
			// indirectly enabled, we treat it as having all platforms enabled.
			// To fix it properly, we will need to have plugins track for which platforms they are enabled,
			// and query the pluginmanager here instead of querying the PluginReference directly.
			if (PluginReference->bEnabled || PluginReference->bEnabled == Plugin.IsEnabled())
			{
				bool bEnabledForProject = PluginReference->IsEnabledForPlatform(Platform) &&
					(Configuration == EBuildConfiguration::Unknown || PluginReference->IsEnabledForTargetConfiguration(Configuration)) &&
					PluginReference->IsEnabledForTarget(TargetType);
				if (!bEnabledForProject)
				{
					return false;
				}
			}
		}
	}
	return true;
}

bool FTargetPlatformBase::RequiresTempTarget(bool bProjectHasCode, EBuildConfiguration Configuration, bool bRequiresAssetNativization, FText& OutReason) const
{
	// check to see if we already have a Target.cs file
	if (bProjectHasCode)
	{
		return false;
	}

	// check if asset nativization is enabled
	if (bRequiresAssetNativization)
    {
		OutReason = LOCTEXT("TempTarget_Nativization", "asset nativization is enabled");
        return true;
    }

	// check to see if any projectmutator modular features are available
	for (IProjectBuildMutatorFeature* Feature : IModularFeatures::Get().GetModularFeatureImplementations<IProjectBuildMutatorFeature>(PROJECT_BUILD_MUTATOR_FEATURE))
	{
		if (Feature->RequiresProjectBuild(PlatformInfo->Name, OutReason))
		{
			return true;
		}
	}

	// check the target platforms for any differences in build settings or additional plugins
	const FProjectDescriptor* Project = IProjectManager::Get().GetCurrentProject();
	if (!FApp::IsEngineInstalled() && !HasDefaultBuildSettings())
	{
		OutReason = LOCTEXT("TempTarget_NonDefaultBuildConfig", "project has non-default build configuration");
		return true;
	}

	// check if there's a non-default plugin change
	FText Reason;
	if (IPluginManager::Get().RequiresTempTargetForCodePlugin(Project, GetPlatformInfo().UBTPlatformString, Configuration, PlatformInfo->PlatformType, Reason))
	{
		OutReason = Reason;
		return true;
	}

	return false;
}

bool FTargetPlatformBase::IsEnabledForPlugin(const IPlugin& Plugin) const
{
	const FProjectDescriptor* Project = IProjectManager::Get().GetCurrentProject();
	return IsPluginEnabledForTarget(Plugin, Project, GetPlatformInfo().UBTPlatformString, EBuildConfiguration::Unknown,
		GetRuntimePlatformType());
}

TSharedPtr<IDeviceManagerCustomPlatformWidgetCreator> FTargetPlatformBase::GetCustomWidgetCreator() const
{
	static TSharedPtr<FDeviceBrowserDefaultPlatformWidgetCreator> DefaultWidgetCreator = MakeShared<FDeviceBrowserDefaultPlatformWidgetCreator>();
	return DefaultWidgetCreator;
}

bool FTargetPlatformBase::HasDefaultBuildSettings() const
{
	// first check default build settings for all platforms
	TArray<FString> BoolKeys, IntKeys, StringKeys, BuildKeys;
	BuildKeys.Add(TEXT("bCompileApex")); 
	BuildKeys.Add(TEXT("bCompileICU"));
	BuildKeys.Add(TEXT("bCompileSimplygon")); 
	BuildKeys.Add(TEXT("bCompileSimplygonSSF"));
	BuildKeys.Add(TEXT("bCompileRecast")); 
	BuildKeys.Add(TEXT("bCompileSpeedTree"));
	BuildKeys.Add(TEXT("bCompileWithPluginSupport")); 
	BuildKeys.Add(TEXT("bCompilePhysXVehicle")); 
	BuildKeys.Add(TEXT("bCompileFreeType"));
	BuildKeys.Add(TEXT("bCompileForSize"));	
	BuildKeys.Add(TEXT("bCompileCEF3")); 
	BuildKeys.Add(TEXT("bCompileCustomSQLitePlatform"));

	if (!DoProjectSettingsMatchDefault(IniPlatformName(), TEXT("/Script/BuildSettings.BuildSettings"), &BuildKeys, nullptr, nullptr))
	{
		return false;
	}

	FString PlatformSection;
	GetBuildProjectSettingKeys(PlatformSection, BoolKeys, IntKeys, StringKeys);

	if(!DoProjectSettingsMatchDefault(IniPlatformName(), PlatformSection, &BoolKeys, &IntKeys, &StringKeys))
	{
		return false;
	}

	return true;
}

bool FTargetPlatformBase::DoProjectSettingsMatchDefault(const FString& InPlatformName, const FString& InSection, const TArray<FString>* InBoolKeys, const TArray<FString>* InIntKeys, const TArray<FString>* InStringKeys)
{
	FConfigFile ProjIni;
	FConfigFile DefaultIni;
	FConfigCacheIni::LoadLocalIniFile(ProjIni, TEXT("Engine"), true, *InPlatformName, true);
	FConfigCacheIni::LoadExternalIniFile(DefaultIni, TEXT("Engine"), *FPaths::EngineConfigDir(), *FPaths::EngineConfigDir(), true, NULL, true);

	if (InBoolKeys != NULL)
	{
		for (int Index = 0; Index < InBoolKeys->Num(); ++Index)
		{
			FString Default(TEXT("False")), Project(TEXT("False"));
			DefaultIni.GetString(*InSection, *((*InBoolKeys)[Index]), Default);
			ProjIni.GetString(*InSection, *((*InBoolKeys)[Index]), Project);
			if (Default.Compare(Project, ESearchCase::IgnoreCase))
			{
				return false;
			}
		}
	}

	if (InIntKeys != NULL)
	{
		for (int Index = 0; Index < InIntKeys->Num(); ++Index)
		{
			int64 Default(0), Project(0);
			DefaultIni.GetInt64(*InSection, *((*InIntKeys)[Index]), Default);
			ProjIni.GetInt64(*InSection, *((*InIntKeys)[Index]), Project);
			if (Default != Project)
			{
				return false;
			}
		}
	}

	if (InStringKeys != NULL)
	{
		for (int Index = 0; Index < InStringKeys->Num(); ++Index)
		{
			FString Default(TEXT("False")), Project(TEXT("False"));
			DefaultIni.GetString(*InSection, *((*InStringKeys)[Index]), Default);
			ProjIni.GetString(*InSection, *((*InStringKeys)[Index]), Project);
			if (Default.Compare(Project, ESearchCase::IgnoreCase))
			{
				return false;
			}
		}
	}

	return true;
}


FTargetPlatformBase::FTargetPlatformBase(const PlatformInfo::FTargetPlatformInfo* const InPlatformInfo) : PlatformInfo(InPlatformInfo)
{
	checkf(PlatformInfo, TEXT("Null PlatformInfo was passed to FTargetPlatformBase. Check the static IsUsable function before creating this object. See FWindowsTargetPlatformModule::GetTargetPlatform()"));

	PlatformOrdinal = AssignPlatformOrdinal(*this);

#if WITH_ENGINE
	// Build Audio Format Settings, Using long form equiv of GetConfigSysten to avoid calling a virtual
	AudioFormatSettings = MakePimpl<Audio::FAudioFormatSettings>(
		FConfigCacheIni::ForPlatform(InPlatformInfo->IniPlatformName), GEngineIni, InPlatformInfo->IniPlatformName.ToString());
#endif //WITH_ENGINE

}

#if WITH_ENGINE

const Audio::FAudioFormatSettings& FTargetPlatformBase::GetAudioFormatSettings() const
{
	check(AudioFormatSettings.IsValid())
	return *AudioFormatSettings;
}

FName FTargetPlatformBase::GetWaveFormat(const class USoundWave* InWave) const
{
	return GetAudioFormatSettings().GetWaveFormat(InWave);
}

void FTargetPlatformBase::GetAllWaveFormats(TArray<FName>& OutFormats) const
{
	GetAudioFormatSettings().GetAllWaveFormats(OutFormats);
}

void FTargetPlatformBase::GetWaveFormatModuleHints(TArray<FName>& OutModuleNames) const
{
	GetAudioFormatSettings().GetWaveFormatModuleHints(OutModuleNames);
}

void FTargetPlatformBase::GetTextureSizeLimits(uint64 & OutMaximumSurfaceBytes, uint64 & OutMaximumPackageBytes) const
{
	FTargetPlatformControlsBase::GetTextureSizeLimitsDefault(GetConfigSystem(),OutMaximumSurfaceBytes,OutMaximumPackageBytes);
}

#endif // WITH_ENGINE

PRAGMA_ENABLE_DEPRECATION_WARNINGS

#undef LOCTEXT_NAMESPACE
