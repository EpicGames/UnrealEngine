// Copyright Epic Games, Inc. All Rights Reserved.
// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/TargetPlatformControlsBase.h"
#include "Interfaces/IProjectBuildMutatorFeature.h"
#include "ProjectDescriptor.h"
#include "Interfaces/IProjectManager.h"
#include "Interfaces/IPluginManager.h"
#include "AnalyticsEventAttribute.h"
#include "Sound/AudioFormatSettings.h"
#include "DeviceBrowserDefaultPlatformWidgetCreator.h"
#include "Features/IModularFeatures.h"
#include "Misc/App.h"
#include "HAL/IConsoleManager.h"

#define LOCTEXT_NAMESPACE "TargetPlatform"

void FTargetPlatformControlsBase::GetPlatformSpecificProjectAnalytics(TArray<FAnalyticsEventAttribute>& AnalyticsParamArray) const
{
	static IConsoleVariable* CVarDesktopForwardShading = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ForwardShading"));
	const bool bRForwardShading = CVarDesktopForwardShading ? (CVarDesktopForwardShading->GetInt() != 0) : false;

	static IConsoleVariable* CVarMobileHdr = IConsoleManager::Get().FindConsoleVariable(TEXT("r.MobileHDR"));
	const bool bRMobileHdr = CVarMobileHdr ? (CVarMobileHdr->GetInt() != 0) : false;

	static IConsoleVariable* CVarInstancedStereo = IConsoleManager::Get().FindConsoleVariable(TEXT("vr.InstancedStereo"));
	const bool bVrInstancedStereo = CVarInstancedStereo ? (CVarInstancedStereo->GetInt() != 0) : false;

	static IConsoleVariable* CVarMobileMultiView = IConsoleManager::Get().FindConsoleVariable(TEXT("vr.MobileMultiView"));
	const bool bVrMobileMultiView = CVarMobileMultiView ? (CVarMobileMultiView->GetInt() != 0) : false;

	static IConsoleVariable* CVarAllowStaticLighting = IConsoleManager::Get().FindConsoleVariable(TEXT("r.AllowStaticLighting"));
	const bool bRAllowStaticLighting = CVarAllowStaticLighting ? (CVarAllowStaticLighting->GetInt() != 0) : false;

	AppendAnalyticsEventAttributeArray(AnalyticsParamArray,
		TEXT("UsesDistanceFields"), TargetPlatformSettings->UsesDistanceFields(),
		TEXT("UsesForwardShading"), TargetPlatformSettings->UsesForwardShading(),
		// TP settings sometimes take value from r.ForwardShading, but some platforms have their own settings, hence adding 
		TEXT("RForwardShading"), bRForwardShading,
		TEXT("RMobileHdr"), bRMobileHdr,
		TEXT("VrInstancedStereo"), bVrInstancedStereo,
		TEXT("VrMobileMultiView"), bVrMobileMultiView,
		TEXT("RAllowStaticLighting"), bRAllowStaticLighting
	);
}

void FTargetPlatformControlsBase::AppendAnalyticsEventConfigBool(TArray<FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride)
{
	bool ConfigValue;
	if (GConfig->GetBool(ConfigSection, ConfigKey, ConfigValue, IniFileName))
	{
		AnalyticsParamArray.Add(FAnalyticsEventAttribute(AnalyticsKeyNameOverride ? AnalyticsKeyNameOverride : ConfigKey, ConfigValue));
	}
}

void FTargetPlatformControlsBase::AppendAnalyticsEventConfigInt(TArray<FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride)
{
	int32 ConfigValue;
	if (GConfig->GetInt(ConfigSection, ConfigKey, ConfigValue, IniFileName))
	{
		AnalyticsParamArray.Add(FAnalyticsEventAttribute(AnalyticsKeyNameOverride ? AnalyticsKeyNameOverride : ConfigKey, ConfigValue));
	}
}

void FTargetPlatformControlsBase::AppendAnalyticsEventConfigFloat(TArray<FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride)
{
	float ConfigValue;
	if (GConfig->GetFloat(ConfigSection, ConfigKey, ConfigValue, IniFileName))
	{
		AnalyticsParamArray.Add(FAnalyticsEventAttribute(AnalyticsKeyNameOverride ? AnalyticsKeyNameOverride : ConfigKey, ConfigValue));
	}
}

void FTargetPlatformControlsBase::AppendAnalyticsEventConfigString(TArray<FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride)
{
	FString ConfigValue;
	if (GConfig->GetString(ConfigSection, ConfigKey, ConfigValue, IniFileName))
	{
		AnalyticsParamArray.Add(FAnalyticsEventAttribute(AnalyticsKeyNameOverride ? AnalyticsKeyNameOverride : ConfigKey, ConfigValue));
	}
}

void FTargetPlatformControlsBase::AppendAnalyticsEventConfigArray(TArray<FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride)
{
	TArray<FString> ConfigValue;
	if (GConfig->GetArray(ConfigSection, ConfigKey, ConfigValue, IniFileName))
	{
		AnalyticsParamArray.Add(FAnalyticsEventAttribute(AnalyticsKeyNameOverride ? AnalyticsKeyNameOverride : ConfigKey, ConfigValue));
	}
}

bool FTargetPlatformControlsBase::IsPluginEnabledForTarget(const IPlugin& Plugin, const FProjectDescriptor* Project, const FString& Platform, EBuildConfiguration Configuration, EBuildTargetType TargetType)
{
	if (!Plugin.GetDescriptor().SupportsTargetPlatform(Platform))
	{
		return false;
	}

	// TODO: Support transitive calculation of per-platform disabling for plugins.
	// Plugins can reference other plugins, and it would be nice to be able to automatically disable for platform X
	// plugins that are only referenced through another plugin that is disabled for platform X.
	// For the time-being, to disable a transitively referenced plugin per-platform, the project has to 
	// directly include the plugin.
	IPluginManager& PluginManager = IPluginManager::Get();

	if (Project != nullptr)
	{
		const FString& PluginName = Plugin.GetName();
		const FPluginReferenceDescriptor* PluginReference = Project->Plugins.FindByPredicate(
			[&PluginName](const FPluginReferenceDescriptor& ExistingReference)
			{
				return ExistingReference.Name == PluginName;
			});
		if (PluginReference)
		{
			// TODO: Remove this workaround for indirect plugin references. A project can mark a plugin as
			// "Enabled": false, but that merely prevents a direct reference, and the plugin might be referenced and
			// enabled by other plugins. PluginReference->IsEnabledForPlatform, IsEnabledForTargetConfiguration, and
			// IsEnabledForTarget will all return false in that case, even though the plugin is actually enabled.
			// Other systems using IPluginManager::Get().GetEnabledPlugins will disagree with the disabled result.
			// To workaround it, when we detect the case of a disabled plugin reference for a plugin that is
			// indirectly enabled, we treat it as having all platforms enabled.
			// To fix it properly, we will need to have plugins track for which platforms they are enabled,
			// and query the pluginmanager here instead of querying the PluginReference directly.
			if (PluginReference->bEnabled || PluginReference->bEnabled == Plugin.IsEnabled())
			{
				bool bEnabledForProject = PluginReference->IsEnabledForPlatform(Platform) &&
					(Configuration == EBuildConfiguration::Unknown || PluginReference->IsEnabledForTargetConfiguration(Configuration)) &&
					PluginReference->IsEnabledForTarget(TargetType);
				if (!bEnabledForProject)
				{
					return false;
				}
			}
		}
	}
	return true;
}

bool FTargetPlatformControlsBase::RequiresTempTarget(bool bProjectHasCode, EBuildConfiguration Configuration, bool bRequiresAssetNativization, FText& OutReason) const
{
	// check to see if we already have a Target.cs file
	if (bProjectHasCode)
	{
		return false;
	}

	// check if asset nativization is enabled
	if (bRequiresAssetNativization)
	{
		OutReason = LOCTEXT("TempTarget_Nativization", "asset nativization is enabled");
		return true;
	}

	// check to see if any projectmutator modular features are available
	for (IProjectBuildMutatorFeature* Feature : IModularFeatures::Get().GetModularFeatureImplementations<IProjectBuildMutatorFeature>(PROJECT_BUILD_MUTATOR_FEATURE))
	{
		if (Feature->RequiresProjectBuild(PlatformInfo->Name, OutReason))
		{
			return true;
		}
	}

	// check the target platforms for any differences in build settings or additional plugins
	const FProjectDescriptor* Project = IProjectManager::Get().GetCurrentProject();
	if (!FApp::IsEngineInstalled() && !HasDefaultBuildSettings())
	{
		OutReason = LOCTEXT("TempTarget_NonDefaultBuildConfig", "project has non-default build configuration");
		return true;
	}

	// check if there's a non-default plugin change
	FText Reason;
	if (IPluginManager::Get().RequiresTempTargetForCodePlugin(Project, GetPlatformInfo().UBTPlatformString, Configuration, PlatformInfo->PlatformType, Reason))
	{
		OutReason = Reason;
		return true;
	}

	return false;
}

bool FTargetPlatformControlsBase::IsEnabledForPlugin(const IPlugin& Plugin) const
{
	const FProjectDescriptor* Project = IProjectManager::Get().GetCurrentProject();
	return IsPluginEnabledForTarget(Plugin, Project, GetPlatformInfo().UBTPlatformString, EBuildConfiguration::Unknown,
		GetRuntimePlatformType());
}

TSharedPtr<IDeviceManagerCustomPlatformWidgetCreator> FTargetPlatformControlsBase::GetCustomWidgetCreator() const
{
	static TSharedPtr<FDeviceBrowserDefaultPlatformWidgetCreator> DefaultWidgetCreator = MakeShared<FDeviceBrowserDefaultPlatformWidgetCreator>();
	return DefaultWidgetCreator;
}

bool FTargetPlatformControlsBase::HasDefaultBuildSettings() const
{
	// first check default build settings for all platforms
	TArray<FString> BoolKeys, IntKeys, StringKeys, BuildKeys;
	BuildKeys.Add(TEXT("bCompileApex"));
	BuildKeys.Add(TEXT("bCompileICU"));
	BuildKeys.Add(TEXT("bCompileSimplygon"));
	BuildKeys.Add(TEXT("bCompileSimplygonSSF"));
	BuildKeys.Add(TEXT("bCompileRecast"));
	BuildKeys.Add(TEXT("bCompileSpeedTree"));
	BuildKeys.Add(TEXT("bCompileWithPluginSupport"));
	BuildKeys.Add(TEXT("bCompilePhysXVehicle"));
	BuildKeys.Add(TEXT("bCompileFreeType"));
	BuildKeys.Add(TEXT("bCompileForSize"));
	BuildKeys.Add(TEXT("bCompileCEF3"));
	BuildKeys.Add(TEXT("bCompileCustomSQLitePlatform"));

	if (!DoProjectSettingsMatchDefault(TargetPlatformSettings->IniPlatformName(), TEXT("/Script/BuildSettings.BuildSettings"), &BuildKeys, nullptr, nullptr))
	{
		return false;
	}

	FString PlatformSection;
	GetBuildProjectSettingKeys(PlatformSection, BoolKeys, IntKeys, StringKeys);

	if (!DoProjectSettingsMatchDefault(TargetPlatformSettings->IniPlatformName(), PlatformSection, &BoolKeys, &IntKeys, &StringKeys))
	{
		return false;
	}

	return true;
}

bool FTargetPlatformControlsBase::DoProjectSettingsMatchDefault(const FString& InPlatformName, const FString& InSection, const TArray<FString>* InBoolKeys, const TArray<FString>* InIntKeys, const TArray<FString>* InStringKeys)
{
	FConfigFile ProjIni;
	FConfigFile DefaultIni;
	FConfigCacheIni::LoadLocalIniFile(ProjIni, TEXT("Engine"), true, *InPlatformName, true);
	FConfigCacheIni::LoadExternalIniFile(DefaultIni, TEXT("Engine"), *FPaths::EngineConfigDir(), *FPaths::EngineConfigDir(), true, NULL, true);

	if (InBoolKeys != NULL)
	{
		for (int Index = 0; Index < InBoolKeys->Num(); ++Index)
		{
			FString Default(TEXT("False")), Project(TEXT("False"));
			DefaultIni.GetString(*InSection, *((*InBoolKeys)[Index]), Default);
			ProjIni.GetString(*InSection, *((*InBoolKeys)[Index]), Project);
			if (Default.Compare(Project, ESearchCase::IgnoreCase))
			{
				return false;
			}
		}
	}

	if (InIntKeys != NULL)
	{
		for (int Index = 0; Index < InIntKeys->Num(); ++Index)
		{
			int64 Default(0), Project(0);
			DefaultIni.GetInt64(*InSection, *((*InIntKeys)[Index]), Default);
			ProjIni.GetInt64(*InSection, *((*InIntKeys)[Index]), Project);
			if (Default != Project)
			{
				return false;
			}
		}
	}

	if (InStringKeys != NULL)
	{
		for (int Index = 0; Index < InStringKeys->Num(); ++Index)
		{
			FString Default(TEXT("False")), Project(TEXT("False"));
			DefaultIni.GetString(*InSection, *((*InStringKeys)[Index]), Default);
			ProjIni.GetString(*InSection, *((*InStringKeys)[Index]), Project);
			if (Default.Compare(Project, ESearchCase::IgnoreCase))
			{
				return false;
			}
		}
	}

	return true;
}

FTargetPlatformControlsBase::FTargetPlatformControlsBase(const PlatformInfo::FTargetPlatformInfo* const InPlatformInfo, ITargetPlatformSettings* TargetPlatformSettings)
	: ITargetPlatformControls(TargetPlatformSettings)
	, PlatformInfo(InPlatformInfo)
{
	checkf(PlatformInfo, TEXT("Null PlatformInfo was passed to FTargetPlatformControlsBase. Check the static IsUsable function before creating this object. See FWindowsTargetPlatformModule::GetTargetPlatform()"));

	PlatformOrdinal = AssignPlatformOrdinal(*this);

#if WITH_ENGINE
	// Build Audio Format Settings, Using long form equiv of GetConfigSysten to avoid calling a virtual
	AudioFormatSettings = MakePimpl<Audio::FAudioFormatSettings>(
		FConfigCacheIni::ForPlatform(InPlatformInfo->IniPlatformName), GEngineIni, InPlatformInfo->IniPlatformName.ToString());
#endif //WITH_ENGINE

}

#if WITH_ENGINE

const Audio::FAudioFormatSettings& FTargetPlatformControlsBase::GetAudioFormatSettings() const
{
	check(AudioFormatSettings.IsValid())
		return *AudioFormatSettings;
}

FName FTargetPlatformControlsBase::GetWaveFormat(const class USoundWave* InWave) const
{
	return GetAudioFormatSettings().GetWaveFormat(InWave);
}

void FTargetPlatformControlsBase::GetAllWaveFormats(TArray<FName>& OutFormats) const
{
	GetAudioFormatSettings().GetAllWaveFormats(OutFormats);
}

void FTargetPlatformControlsBase::GetWaveFormatModuleHints(TArray<FName>& OutModuleNames) const
{
	GetAudioFormatSettings().GetWaveFormatModuleHints(OutModuleNames);
}

/* static */ void FTargetPlatformControlsBase::GetTextureSizeLimitsDefault(FConfigCacheIni* ConfigSystem,uint64 & OutMaximumSurfaceBytes, uint64 & OutMaximumPackageBytes)
{
	OutMaximumSurfaceBytes = 1ULL << 31; // 2 GB
	//OutMaximumPackageBytes = 1ULL << 32; // 4 GB  seems to work on some platforms
	OutMaximumPackageBytes = 1ULL << 31; // 2 GB

	#if 0
	// for stress testing
	OutMaximumSurfaceBytes = 32 * 1024 * 1024;
	OutMaximumPackageBytes = OutMaximumSurfaceBytes * 2;
	#endif

	int64 MaxChunkSize = 0;
	if ( ConfigSystem->GetInt64(TEXT("/Script/UnrealEd.ProjectPackagingSettings"), TEXT("MaxChunkSize"), MaxChunkSize, GGameIni) &&
		MaxChunkSize != 0 )
	{
		check( MaxChunkSize > 0 );

		OutMaximumPackageBytes = FMath::Min<uint64>(OutMaximumPackageBytes,MaxChunkSize);
	}

	OutMaximumSurfaceBytes = FMath::Min<uint64>(OutMaximumSurfaceBytes,OutMaximumPackageBytes);
}

#endif // WITH_ENGINE

#undef LOCTEXT_NAMESPACE