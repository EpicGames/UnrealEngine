// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "HAL/FileManager.h"
#include "Misc/App.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CommandLine.h"
#include "Misc/Paths.h"
#include "Misc/OutputDeviceRedirector.h"
#include "ShaderCompilerCore.h"
#include "Stats/Stats.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/MonitoredProcess.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformControls.h"
#include "Interfaces/ITargetPlatformSettings.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "Interfaces/ITargetPlatformControlsModule.h"
#include "Interfaces/ITargetPlatformSettingsModule.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/IAudioFormat.h"
#include "Interfaces/IAudioFormatModule.h"
#include "Interfaces/IShaderFormat.h"
#include "Interfaces/IShaderFormatModule.h"
#include "Interfaces/ITextureFormat.h"
#include "Interfaces/ITextureFormatManagerModule.h"
#include "Interfaces/ITextureFormatModule.h"
#include "Logging/StructuredLog.h"
#include "PlatformInfo.h"
#include "DesktopPlatformModule.h"
#include "Interfaces/ITurnkeySupportModule.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "Misc/FileHelper.h"

DEFINE_LOG_CATEGORY_STATIC(LogTargetPlatformManager, Log, All);

// AutoSDKs needs the extra DDPI info
#ifndef AUTOSDKS_ENABLED
#define AUTOSDKS_ENABLED DDPI_HAS_EXTENDED_PLATFORMINFO_DATA
#endif

namespace UE::AutoSDK
{
	static const TCHAR* SDKRootEnvVar = TEXT("UE_SDKS_ROOT");
	static const TCHAR* SDKInstallManifestFileName = TEXT("CurrentlyInstalled.txt");
	static const TCHAR* SDKLastScriptRunVersionFileName = TEXT("CurrentlyInstalled.Version.txt");
	static const TCHAR* SDKRequiredScriptVersionFileName = TEXT("Version.txt");
	static const TCHAR* SDKEnvironmentVarsFile = TEXT("OutputEnvVars.txt");
}

static const TCHAR* SkipUBTSDKSetupEnvVar = TEXT("UE_SKIP_UBT_SDK_SETUP");
static FProcHandle SDKSetupUBTProc;

static FConfigFile& GetCachedInfoToSaveAfterUBT()
{
	static FConfigFile CachedInfoToSaveAfterUBT;
	return CachedInfoToSaveAfterUBT;
}

static bool IsAutoSDKsEnabled()
{
#if AUTOSDKS_ENABLED
	FString SDKPath = FPlatformMisc::GetEnvironmentVariable(UE::AutoSDK::SDKRootEnvVar);

	// AutoSDKs only enabled if UE_SDKS_ROOT is set.
	if (SDKPath.Len() != 0)
	{
		return true;
	}
#endif
	return false;
}

static FString GetProjectPathForUBT()
{
	if (FPaths::IsProjectFilePathSet())
	{
		return FPaths::ConvertRelativePathToFull(FPaths::GetProjectFilePath());
	}
	if (FApp::HasProjectName())
	{
		FString ProjectPath = FPaths::ProjectDir() / FApp::GetProjectName() + TEXT(".uproject");
		if (FPaths::FileExists(ProjectPath))
		{
			return ProjectPath;
		}
	}
	return FString();
}

// get a value from an SDK.json file
static FString GetSDKInfo(const TCHAR* Platform, const TCHAR* Key)
{
	// 1. start with per-project json
	//   a. look in platform extension location, then non-platext location, for a file
	//   b. if key not found, look up the ParentSdkFile chain (which is relative paths)
	// 2. if key still not found, go back to 1. but using engine-wide SDK.json file

	auto MakeConfigFilename = [Platform](const FString& RootDir) -> FString
		{
			FString PlatformExtensionLocation = FString::Printf(TEXT("%s/Platforms/%s/Config/%s_SDK.json"), *RootDir, Platform, Platform);
			if (FPaths::FileExists(PlatformExtensionLocation))
			{
				return PlatformExtensionLocation;
			}
			FString StandardLocation = FString::Printf(TEXT("%s/Config/%s/%s_SDK.json"), *RootDir, Platform, Platform);
			if (FPaths::FileExists(StandardLocation))
			{
				return StandardLocation;
			}
			return TEXT("");
		};

	auto GetJsonObject = [](const FString& Filename) -> TSharedPtr<FJsonObject>
		{
			FString FileContents;
			if (!FFileHelper::LoadFileToString(FileContents, *Filename))
			{
				return nullptr;
			}

			TSharedPtr<FJsonObject> Object;
			TSharedRef<TJsonReader<> > Reader = TJsonReaderFactory<>::Create(FileContents);
			if (!FJsonSerializer::Deserialize(Reader, Object) || !Object.IsValid())
			{
				return nullptr;
			}

			return Object;
		};

	// Steps 1 or 2 above
	for (int Pass = 0; Pass < 2; Pass++)
	{
		// Step 1 - ProjectDir; Step 2 - EngineDir
		FString RootDir = Pass == 0 ? GetProjectPathForUBT() : FPaths::EngineDir();
		// Step a. Choose platformextension or standard location
		FString InfoFile = RootDir.IsEmpty() ? FString() : MakeConfigFilename(RootDir);

		// make sure we found a file
		if (InfoFile.IsEmpty())
		{
			continue;
		}

		bool bContinueUpToParent;
		do
		{
			bContinueUpToParent = false;

			// read the file and get the key if it exists
			TSharedPtr<FJsonObject> JsonObj = GetJsonObject(InfoFile);
			if (JsonObj != nullptr)
			{
				if (JsonObj->HasField(Key))
				{
					// if we have a value, we are done!
					return JsonObj->GetStringField(Key);
				}
				// step b. go up parent chain
				if (JsonObj->HasField(TEXT("ParentSDKFile")))
				{
					// the parent field is relative to current json file
					InfoFile = FPaths::GetPath(InfoFile) / JsonObj->GetStringField(TEXT("ParentSDKFile"));
					FPaths::NormalizeFilename(InfoFile);

					bContinueUpToParent = true;
				}
			}
			else
			{
				UE_LOGF(LogTargetPlatformManager, Error, "Failed to parse SDK json file '%ls'", *InfoFile);
			}
		} while (bContinueUpToParent);
	}

	// if we got no value, then return empty string for failure
	return FString();
}

static FString GetAutoSDKPlatformDir(const TCHAR* Platform)
{
	FString SDKPath = FPlatformMisc::GetEnvironmentVariable(UE::AutoSDK::SDKRootEnvVar);
#if PLATFORM_WINDOWS
	static FString HostPlatform(TEXT("HostWin64"));
#else
	static FString HostPlatform = FString::Printf(TEXT("Host%hs"), FPlatformProperties::IniPlatformName());
#endif		
	FString AutoSDKPlatform = GetSDKInfo(Platform, TEXT("AutoSDKPlatform"));

	return FPaths::Combine(*SDKPath, *HostPlatform, AutoSDKPlatform.Len() > 0 ? *AutoSDKPlatform : Platform);
}

// Check to see if any platforms are not up to date with AutoSDK setup. This is how it's determined:
// 1. Load an existing <proj>/Intermediate/CachedAutoSdkInfo.ini to get cached info from last run
// 2. -> Compare current vs cached value for UE_SDKS_ROOT
// 3. For each (real) platform:
//   a. Get the current AutoSDKVersion from <platform>_SDK.json (looking in project overrides, and parent SDK.json chain as needed)
//   b. -> Compare current vs cached AutoSDKVersion
//   c. Set TargetSDKRoot to UE_SDKS_ROOT/<Platform>/<AutoSDKVersion>
//   d. Check if TargetSDKRoot/AutoSDKVersion/setup.bat|sh [aka SetupScript] exists
//   e. -> Compare current vs cached SetupScript existence
//   f. Read TargetSDKRoot/CurrentlyInstalled.txt
//   g. Get installed version, and if it's AutoSDK or ManualSDK
//   h. If ManualSDK, skip and move to next platform
//   i. -> Compare installed version to AutoSDKVersion
//   j. -> Compare installed "script version" to required "script version" (this manages updating setup.bat)
//   j. -> Check for existence of TargetSDKRoot/OutputEnvVars.txt
// 4. If anything above (marked with ->) doesn't match up, then run UBT which will update CurrentlyInstalled.txt and OutputEnvVars.txt
// 5. [much later] Read in OutputEnvVars.txt (either from step 4, or a previous UBT run) to set envvars for the editor process
//
// Still todo: 
//   Manage CurrentlyInstalled.Version.txt, which allows for forcing a re-run of the SetupScript, which would mean that we need to run UBT
//   Double check what happens if a user goes from manual->auto or auto->manual
//   We can't tell here if the platform prefers auto vs manual, which may affect the previous line
//   ManualSDKEnvVars.txt? I believe this is not used and we can ignore it
//   Maybe force run UBT if UBT.dll has been updated since last run of editor, just in case
static bool NeedsToRunSetupPlatformsUBT()
{
	int32 SkipUBTSDKSetupEnvVarValue = FCString::Atoi(*FPlatformMisc::GetEnvironmentVariable(SkipUBTSDKSetupEnvVar));
	if (SkipUBTSDKSetupEnvVarValue == 1)
	{
		return false;
	}

	if (!IsAutoSDKsEnabled())
	{
		// UBT is usually always invoked when AutoSDK is disabled to check platform status, as there is no other way to check sdk validity. 
		// This causes a startup overhead and unexpected console log output because we have to launch an UBT process and read the output in order to check platform validity.
		// Returning false here for program target type solves the problem by skipping calling UBT, we assume all platforms are valid.
#if IS_PROGRAM
		return false;
#else
		return true;
#endif
	}

	// early out and always return true on builders
	if (GIsBuildMachine)
	{
		return true;
	}

	SCOPED_BOOT_TIMING("AutoSDKInit - NeedsToRunSetupPlatformsUBT");

	FString CachedAutoSDKPath = FPaths::ProjectIntermediateDir() / TEXT("CachedAutoSdkInfo.ini");
	FConfigFile CachedInfo;
	CachedInfo.Read(CachedAutoSDKPath);

	// first check that the UE_SDKS_ROOT var is the same
	FString SDKsRoot = FPlatformMisc::GetEnvironmentVariable(UE::AutoSDK::SDKRootEnvVar);
	FString CachedValue;
	CachedInfo.GetString(TEXT("Global"), TEXT("SDKsRoot"), CachedValue);
	GetCachedInfoToSaveAfterUBT().SetString(TEXT("Global"), TEXT("SDKsRoot"), *SDKsRoot);

	// even if we need to run, we finish this function to get all the cached info to check next run
	bool bNeedsToRun = false;
	if (SDKsRoot != CachedValue)
	{
		UE_LOGF(LogTargetPlatformManager, Log, "Running UBT for AutoSDK init because the value of %ls changed", UE::AutoSDK::SDKRootEnvVar);
		bNeedsToRun = true;
	}

	for (const TPair<FName, FDataDrivenPlatformInfo>& Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
	{
		// check only real platforms, that have an AutoSDK path
		if (!Pair.Value.bEnabledForUse || Pair.Value.bIsFakePlatform || Pair.Value.AutoSDKPath.IsEmpty())
		{
			continue;
		}

		// get the AutoSDK version - AutoSDKDirectory if specified, or use default of MainVersion
		FString PlatformName = Pair.Key.ToString();
		FString AutoSDKVersion = GetSDKInfo(*PlatformName, TEXT("AutoSDKDirectory"));
		if (AutoSDKVersion.IsEmpty())
		{
			AutoSDKVersion = GetSDKInfo(*PlatformName, TEXT("MainVersion"));
		}
		CachedInfo.GetString(*PlatformName, TEXT("AutoSDKDirectory"), CachedValue);
		GetCachedInfoToSaveAfterUBT().SetString(*PlatformName, TEXT("AutoSDKDirectory"), *AutoSDKVersion);

		if (AutoSDKVersion != CachedValue)
		{
			UE_LOGF(LogTargetPlatformManager, Log, "Running UBT for AutoSDK init because AutoSDK version for %ls changed", *PlatformName);
			bNeedsToRun = true;
		}

		FString TargetSDKRoot = GetAutoSDKPlatformDir(*PlatformName);

		// check if presence of setup.bat, etc matches
#if PLATFORM_WINDOWS
		FString SetupScript = TargetSDKRoot / AutoSDKVersion / "setup.bat";
#else
		FString SetupScript = TargetSDKRoot / AutoSDKVersion / "setup.sh";
#endif
		bool bSetupScriptExists = FPaths::FileExists(SetupScript);
		bool bCachedValue;
		CachedInfo.GetBool(*PlatformName, TEXT("bSetupScriptExists"), bCachedValue);
		GetCachedInfoToSaveAfterUBT().SetBool(*PlatformName, TEXT("bSetupScriptExists"), bSetupScriptExists);
		if (bSetupScriptExists != bCachedValue)
		{
			UE_LOGF(LogTargetPlatformManager, Log, "Running UBT for AutoSDK init because the existence of %ls changed", *SetupScript);
			bNeedsToRun = true;
		}

		// only deal with autosdk stuff if the setupscript actually exists now - if it doesn't, none of this will do anything in UBT
		if (bSetupScriptExists)
		{
			// check the currently installed manifest file - if the version doesn't match and it's not a ManualSDK installation,
			// we will need to run UBT
			TArray<FString> FileLines;
			bool bIsManualInstall = false;
			if (FFileHelper::LoadFileToStringArray(FileLines, *(TargetSDKRoot / UE::AutoSDK::SDKInstallManifestFileName)) &&
				FileLines.Num() > 1)
			{
				if (FileLines[1] == TEXT("ManualSDK"))
				{
					bIsManualInstall = true;
				}
				if (FileLines[0] != AutoSDKVersion && !bIsManualInstall)
				{
					UE_LOGF(LogTargetPlatformManager, Log, "Running UBT for AutoSDK init because the installed version of %ls doesn't match the required version", *PlatformName);
					bNeedsToRun = true;
				}
			}
			else
			{
				UE_LOGF(LogTargetPlatformManager, Log, "Running UBT for AutoSDK init because the the file %ls is invalid", *(TargetSDKRoot / UE::AutoSDK::SDKInstallManifestFileName));
				bNeedsToRun = true;
			}

			if (!bIsManualInstall)
			{
				// check if the last run script version matches what the AutoSDK version requires
				// if they mismatch, or if the installed version is missing, run UBT
				FString InstalledScriptVersion;
				FString RequiredScriptVersion;
				if (!FFileHelper::LoadFileToString(RequiredScriptVersion, *(TargetSDKRoot / AutoSDKVersion / UE::AutoSDK::SDKRequiredScriptVersionFileName)))
				{
					RequiredScriptVersion = TEXT("UnspecifiedScriptVersion");
				}
				if (!FFileHelper::LoadFileToString(InstalledScriptVersion, *(TargetSDKRoot / UE::AutoSDK::SDKLastScriptRunVersionFileName)) ||
					RequiredScriptVersion.TrimStartAndEnd() != InstalledScriptVersion.TrimStartAndEnd())
				{
					UE_LOGF(LogTargetPlatformManager, Log, "Running UBT for AutoSDK init because the last run script version in %ls doesn't match required (%ls !- %ls)", *(TargetSDKRoot / UE::AutoSDK::SDKLastScriptRunVersionFileName), *InstalledScriptVersion, *RequiredScriptVersion);
					bNeedsToRun = true;
				}
			}

			// autosdk writes out the OUtputEnvvars.txt file that should exists if autosdk ran
			if (!bIsManualInstall && !FPaths::FileExists(*(TargetSDKRoot / UE::AutoSDK::SDKEnvironmentVarsFile)))
			{
				UE_LOGF(LogTargetPlatformManager, Log, "Running UBT for AutoSDK init because the file %ls is missing", *(TargetSDKRoot / UE::AutoSDK::SDKEnvironmentVarsFile));
				bNeedsToRun = true;
			}
		}
	}

	UE_LOGFMT(LogTargetPlatformManager, Log, "{CachedAutoSDKPath} is {Status}", CachedAutoSDKPath, bNeedsToRun ? "out-of-date" : "up-to-date");
	return bNeedsToRun;
}

void LaunchUBTValidatePlatformsProc(FName PlatformName = NAME_None)
{
	FString UBTParams(TEXT("-Mode=ValidatePlatforms -OutputSDKs "));
	if (PlatformName == NAME_None)
	{
		UBTParams += TEXT(" -AllPlatforms ");
	}
	else
	{
		UBTParams += FString::Printf(TEXT("  -Platforms=%s "), *(PlatformName.ToString()));
	}

	if (FString Project = GetProjectPathForUBT(); Project.Len() > 0)
	{
		UBTParams += FString::Printf(TEXT(" -project=\"%s\""), *Project);
	}
	// Write the output to a separate file since it performs much better than read pipes in this scenario
	// where the invocation script is quite involved and is in turn calling other scripts and programs...
	{
		FString LogFile = FPaths::Combine(FPaths::ProjectLogDir(), "AutoSDKInfo.txt");
		FString AbsLogFile = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*LogFile);
		UBTParams += FString::Printf(TEXT(" -log=\"%s\" -verbose -timestamps"), *AbsLogFile);
	}

	if (FParse::Param(FCommandLine::Get(), TEXT("Multiprocess")) == false && NeedsToRunSetupPlatformsUBT())
	{
		int32 UBTReturnCode = -1;

		void* ReadPipe = nullptr;
		void* WritePipe = nullptr;
		SDKSetupUBTProc = FDesktopPlatformModule::Get()->InvokeUnrealBuildToolAsync(UBTParams, *GLog, ReadPipe, WritePipe, true);
		if (!SDKSetupUBTProc.IsValid())
		{
			UE_LOGF(LogTargetPlatformManager, Warning, "AutoSDK is enabled (UE_SDKS_ROOT is set), but failed to run UBT to check SDK status! Check your installation.");
		}
	}
}

// kick off a call to UBT nice and early so that it's results are hopefully ready when needed
FDelayedAutoRegisterHelper GSDKInit(EDelayedRegisterRunPhase::FileSystemReady, []
	{
		SCOPED_BOOT_TIMING("AutoSDKInit - DelayedAutoRegisterHelper");
		LaunchUBTValidatePlatformsProc();
	}
);

static const size_t MaxPlatformCount = 64;		// In the unlikely event that someone bumps this please note that there's
												// an implicit assumption that there won't be more than 64 unique target
												// platforms in the FTargetPlatformSet code since it uses one bit of an
												// uint64 per platform.

static const ITargetPlatform* TargetPlatformArray[MaxPlatformCount];
static const ITargetPlatformControls* TargetPlatformControlsArray[MaxPlatformCount];

static int32 PlatformCounter = 0;
static int32 PlatformControlsCounter = 0;

int32 ITargetPlatform::AssignPlatformOrdinal(const ITargetPlatform& Platform)
{
	check(PlatformCounter < MaxPlatformCount);

	const int32 Ordinal = PlatformCounter++;

	check(TargetPlatformArray[Ordinal] == nullptr);
	TargetPlatformArray[Ordinal] = &Platform;

	return Ordinal;
}
int32 ITargetPlatformControls::AssignPlatformOrdinal(const ITargetPlatformControls& Platform)
{
	check(PlatformControlsCounter < MaxPlatformCount);

	const int32 Ordinal = PlatformControlsCounter++;

	check(TargetPlatformControlsArray[Ordinal] == nullptr);
	TargetPlatformControlsArray[Ordinal] = &Platform;

	return Ordinal;
}

const ITargetPlatformControls* ITargetPlatformControls::GetPlatformFromOrdinal(int32 Ordinal)
{
	check(Ordinal < PlatformControlsCounter);

	return TargetPlatformControlsArray[Ordinal];
}

const class ITargetPlatformSettings& ITargetDevice::GetPlatformSettings() const
{
	return *(GetTargetPlatform().GetTargetPlatformSettings());
}
const class ITargetPlatformControls& ITargetDevice::GetPlatformControls() const
{
	return *(GetTargetPlatform().GetTargetPlatformControls());
}

ITargetPlatform::FOnTargetDeviceDiscovered& ITargetPlatform::OnDeviceDiscovered()
{
	static FOnTargetDeviceDiscovered Delegate;
	return Delegate;
}

ITargetPlatform::FOnTargetDeviceLost& ITargetPlatform::OnDeviceLost()
{
	static FOnTargetDeviceLost Delegate;
	return Delegate;
}

ITargetPlatformControls::FOnTargetDeviceDiscovered& ITargetPlatformControls::OnDeviceDiscovered()
{
	static FOnTargetDeviceDiscovered Delegate;
	return Delegate;
}

ITargetPlatformControls::FOnTargetDeviceLost& ITargetPlatformControls::OnDeviceLost()
{
	static FOnTargetDeviceLost Delegate;
	return Delegate;
}

/**
 * Module for the target platform manager
 */
class FTargetPlatformManagerModule
	: public ITargetPlatformManagerModule
{
	void SDKSetup(FName InPlatformName = NAME_None)
	{
		if (SDKSetupUBTProc.IsValid())
		{
			SCOPED_BOOT_TIMING("AutoSDKInit - WaitForUBTProc");
			// wait for UBT to finish
			FPlatformProcess::WaitForProc(SDKSetupUBTProc);

			FString LogFile = FPaths::Combine(FPaths::ProjectLogDir(), "AutoSDKInfo.txt");
			FString AbsLogFile = IFileManager::Get().ConvertToAbsolutePathForExternalAppForWrite(*LogFile);

			TArray<FString> Lines;

			bool bIsAutoSDKEnabled = IsAutoSDKsEnabled();
			if (FFileHelper::LoadFileToStringArray(Lines, *AbsLogFile) && !bIsAutoSDKEnabled)
			{
				TMap<FName, FName> UBTNameToIniPlatformName;
				for (auto Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
				{
					FName PlatformName = Pair.Key;
					if (InPlatformName == PlatformName || InPlatformName == NAME_None)
					{
						const FDataDrivenPlatformInfo& Info = Pair.Value;
						UBTNameToIniPlatformName.Add(Info.UBTPlatformName, PlatformName);
					}
				}

				for (int Index = 0; Index < Lines.Num(); ++Index)
				{
					FString Item = Lines[Index];
					FString ValidateTag = TEXT("##PlatformValidate: ");
					int32 StartIndex = Item.Find(ValidateTag, ESearchCase::IgnoreCase);
					if (StartIndex != INDEX_NONE)
					{
						StartIndex += ValidateTag.Len();
						int32 EndIndex = Item.Find(TEXT(" "), ESearchCase::IgnoreCase, ESearchDir::FromStart, StartIndex);
						bool bIsSDKValid = !Item.Contains(TEXT("INVALID"));
						if (EndIndex != INDEX_NONE)
						{
							FName UBTName = FName(Item.Mid(StartIndex, EndIndex - StartIndex));
							FName* IniPlatformName = UBTNameToIniPlatformName.Find(UBTName);
							if (IniPlatformName != nullptr)
							{
								PlatformsSetup.Add(*IniPlatformName, bIsSDKValid);
							}
						}
					}
				}
			}
			// if it failed, we don't want to cache the saved info, so we make sure we run it again next time
			int32 ReturnCode = 0;

			FPlatformProcess::GetProcReturnCode(SDKSetupUBTProc, &ReturnCode);
			FPlatformProcess::CloseProc(SDKSetupUBTProc);
			SDKSetupUBTProc.Reset();
			if (IsAutoSDKsEnabled() && ReturnCode == 0)
			{
				// now save out the cached info so next run we can skip UBT
				GetCachedInfoToSaveAfterUBT().Write(*(FPaths::ProjectIntermediateDir() / TEXT("CachedAutoSdkInfo.ini")));
			}

			UE_LOGFMT(LogTargetPlatformManager, Log, "UBT AutoSDK ReturnCode: {ReturnCode}", ReturnCode);
		}

		// we have to setup our local environment according to AutoSDKs or the ITargetPlatform's IsSDkInstalled calls may fail
		// before we get a change to setup for a given platform.  Use the platforminfo list to avoid any kind of interdependency.
		SCOPED_BOOT_TIMING("AutoSDKInit - SetupAutoSDK");
		if (IsAutoSDKsEnabled())
		{
			for (auto Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
			{
				FName PlatformName = Pair.Key;
				if (InPlatformName == PlatformName || InPlatformName == NAME_None)
				{
					if (Pair.Value.AutoSDKPath.Len() > 0)
					{
						SetupAutoSDK(Pair.Value.AutoSDKPath);
					}
				}
			}
		}
		else if (!NeedsToRunSetupPlatformsUBT())
		{
			// If AutoSDK is disabled and we didn't run UBT, assume all platforms are valid as there is no way to check.
			for (auto Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
			{
				FName PlatformName = Pair.Key;
				PlatformsSetup.Add(PlatformName, true);
			}
		}

		FString ManualSDKEnvironmentVarsPath = FPaths::EngineIntermediateDir() / FString(TEXT("ManualSDKEnvVars.txt"));
		if (IFileManager::Get().FileExists(*ManualSDKEnvironmentVarsPath))
		{
			SetupEnvironmentFromManualSDK(ManualSDKEnvironmentVarsPath);
		}
	}

public:

	/** Default constructor. */
	FTargetPlatformManagerModule()
		: bRestrictFormatsToRuntimeOnly(false)
		, bForceCacheUpdate(true)
		, bHasInitErrors(false)
		, bIgnoreFirstDelegateCall(true)
	{
	}

	/** Destructor. */
	virtual ~FTargetPlatformManagerModule()
	{
		if (bIsInitialized)
		{
			FModuleManager::Get().OnModulesChanged().RemoveAll(this);
		}
	}


	virtual void Initialize()
	{
		if (bIsInitialized)
		{
			return;
		}

		bIsInitialized = true;

#if WITH_EDITOR && UE_WITH_TURNKEY_SUPPORT
		ITurnkeySupportModule::Get().UpdateSdkInfo();
#endif

		SDKSetup();

		TextureFormatManager = FModuleManager::LoadModulePtr<ITextureFormatManagerModule>("TextureFormat");

		Invalidate();

		FModuleManager::Get().OnModulesChanged().AddRaw(this, &FTargetPlatformManagerModule::ModulesChangesCallback);
	}

public:

	// ITargetPlatformManagerModule interface

	virtual bool HasInitErrors(FString* OutErrorMessages) const
	{
		if (OutErrorMessages)
		{
			*OutErrorMessages = InitErrorMessages;
		}
		return bHasInitErrors;
	}

	virtual void Invalidate() override
	{
		bForceCacheUpdate = true;
		GetActiveTargetPlatforms();

		bForceCacheUpdate = false;
		
		// If we've had an error due to an invalid target platform, don't do additional work
		if (!bHasInitErrors)
		{
			GetAudioFormats();
			GetShaderFormats();
		}

		OnTargetPlatformsInvalidated.Broadcast();
	}

	virtual FOnTargetPlatformsInvalidated& GetOnTargetPlatformsInvalidatedDelegate() override
	{
		return OnTargetPlatformsInvalidated;
	}

	virtual const TArray<ITargetPlatform*>& GetTargetPlatforms() override
	{
		if (Platforms.Num() == 0 || bForceCacheUpdate)
		{
			DiscoverAvailablePlatforms();
		}

		return Platforms;
	}

	virtual const TArray<ITargetPlatformControls*>& GetTargetPlatformControls() override
	{
		if (PlatformControls.Num() == 0 || bForceCacheUpdate)
		{
			DiscoverAvailablePlatforms();
		}

		return PlatformControls;
	}

	virtual const TArray<ITargetPlatformSettings*>& GetTargetPlatformSettings() override
	{
		if (PlatformSettings.Num() == 0 || bForceCacheUpdate)
		{
			DiscoverAvailablePlatforms();
		}

		return PlatformSettings;
	}

	virtual ITargetDevicePtr FindTargetDevice(const FTargetDeviceId& DeviceId) override
	{
		ITargetPlatform* Platform = FindTargetPlatform(DeviceId.GetPlatformName());

		if (Platform != nullptr)
		{
			return Platform->GetDevice(DeviceId);
		}

		return nullptr;
	}

	virtual ITargetPlatform* FindTargetPlatform(FStringView Name) override
	{
		GetTargetPlatforms(); // Populates PlatformsByName

		if (ITargetPlatform** Platform = PlatformsByName.Find(FName(Name)))
		{
			return *Platform;
		}

		return nullptr;
	}

	virtual ITargetPlatform* FindTargetPlatform(FName Name) override
	{
		GetTargetPlatforms(); // Populates PlatformsByName

		if (ITargetPlatform** Platform = PlatformsByName.Find(Name))
		{
			return *Platform;
		}

		return nullptr;
	}

	virtual ITargetPlatform* FindTargetPlatform(const TCHAR* Name) override
	{
		return FindTargetPlatform(FName(Name));
	}

	virtual ITargetPlatform* FindTargetPlatformWithSupport(FName SupportType, FName RequiredSupportedValue)
	{
		// first try to find an active target platform. if that fails, try all target platforms.
		// this gives priority to the active target platform if multiple platforms support the same value
		for (int Pass = 0; Pass < 2; Pass++)
		{
			const TArray<ITargetPlatform*>& TargetPlatforms = (Pass == 0) ? GetActiveTargetPlatforms() : GetTargetPlatforms();

			for (int32 Index = 0; Index < TargetPlatforms.Num(); Index++)
			{
				if (TargetPlatforms[Index]->SupportsValueForType(SupportType, RequiredSupportedValue))
				{
					return TargetPlatforms[Index];
				}
			}
		}

		return nullptr;
	}

	virtual const TArray<ITargetPlatform*>& GetCookingTargetPlatforms() override
	{
		return GetActiveTargetPlatforms();
	}

	virtual const TArray<ITargetPlatform*>& GetActiveTargetPlatforms() override
	{
		static bool bInitialized = false;
		static FRWLock InitializedLock;
		static TArray<ITargetPlatform*> Results;

		// Check if result is already initialized with lightweight read-only lock
		{
			FRWScopeLock InitializedLockGuard(InitializedLock, FRWScopeLockType::SLT_ReadOnly);
			if (bInitialized && !bForceCacheUpdate)
			{
				return Results;
			}
		}

		// Initialize the result now
		{
			FRWScopeLock InitializedLockGuard(InitializedLock, FRWScopeLockType::SLT_Write);
			// Check if another thread initialized the result before this thread received the write lock
			if (!bInitialized || bForceCacheUpdate)
			{
				bInitialized = InitializeActiveTargetPlatforms(Results);
			}
		}

		return Results;
	}

	virtual bool RestrictFormatsToRuntimeOnly() override
	{
		GetActiveTargetPlatforms(); // make sure this is initialized

		return bRestrictFormatsToRuntimeOnly;
	}

	virtual ITargetPlatform* GetRunningTargetPlatform() override
	{
		static bool bInitialized = false;
		static FRWLock InitializedLock;
		static ITargetPlatform* Result = nullptr;

		// Check if result is already initialized with lightweight read-only lock
		{
			FRWScopeLock InitializedLockGuard(InitializedLock, FRWScopeLockType::SLT_ReadOnly);
			if (bInitialized && !bForceCacheUpdate)
			{
				return Result;
			}
		}

		// Initialize the result now
		{
			FRWScopeLock InitializedLockGuard(InitializedLock, FRWScopeLockType::SLT_Write);
			// Check if another thread initialized the result before this thread received the write lock
			if (!bInitialized || bForceCacheUpdate)
			{
				Result = nullptr;

				const TArray<ITargetPlatform*>& TargetPlatforms = GetTargetPlatforms();

				for (int32 Index = 0; Index < TargetPlatforms.Num(); Index++)
				{
					if (TargetPlatforms[Index]->IsRunningPlatform())
					{
						// we should not have two running platforms
						checkf((Result == nullptr),
							TEXT("Found multiple running platforms.\n\t%s\nand\n\t%s"),
							*Result->PlatformName(),
							*TargetPlatforms[Index]->PlatformName()
						);
						Result = TargetPlatforms[Index];
						bInitialized = (Result != nullptr);
					}
				}
			}
		}

		return Result;
	}

	template<typename FormatType, typename FormatModuleType, typename HelperType>
	const TArray<const FormatType*>& GetFormatsWithHints()
	{
		static bool bInitialized = false;
		static FRWLock InitializedLock;
		static TArray<const FormatType*> Results;

		// Check if result is already initialized with lightweight read-only lock
		{
			FRWScopeLock InitializedLockGuard(InitializedLock, FRWScopeLockType::SLT_ReadOnly);
			if (bInitialized && !bForceCacheUpdate)
			{
				return Results;
			}
		}

		// Initialize the result now
		{
			FRWScopeLock InitializedLockGuard(InitializedLock, FRWScopeLockType::SLT_Write);
			// Check if another thread initialized the result before this thread received the write lock
			if (!bInitialized || bForceCacheUpdate)
			{
				InitializeFormatsWithHints<FormatType, FormatModuleType, HelperType>(Results);
				bInitialized = true;
			}
		}

		return Results;
	}

	struct FAudioHintHelper
	{
		static IAudioFormat* GetFormatFromModule(IAudioFormatModule* Module)
		{
			return Module->GetAudioFormat();
		}
		static const TCHAR* GetAllModuleWildcard()
		{
			return TEXT("*AudioFormat*");
		}
		static const TCHAR* GetFormatDesc()
		{
			return TEXT("audio");
		}
#if WITH_ENGINE 
		static void GetHintedModules(ITargetPlatform* Platform, TArray<FName>& Hints)
		{
			Platform->GetWaveFormatModuleHints(Hints);
		}
		static void GetRequiredFormats(ITargetPlatform* Platform, TArray<FName>& RequiredFormats)
		{
			Platform->GetAllWaveFormats(RequiredFormats);
		}
#endif
	};

	struct FShaderHintHelper
	{
		static IShaderFormat* GetFormatFromModule(IShaderFormatModule* Module)
		{
			return Module->GetShaderFormat();
		}
		static const TCHAR* GetAllModuleWildcard()
		{
			return SHADERFORMAT_MODULE_WILDCARD;
		}
		static const TCHAR* GetFormatDesc()
		{
			return TEXT("shader");
		}
#if WITH_ENGINE 
		static void GetHintedModules(ITargetPlatform* Platform, TArray<FName>& Hints)
		{
			Platform->GetShaderFormatModuleHints(Hints);
			Hints.Add(TEXT("ShaderFormatVectorVM"));
		}
		static void GetRequiredFormats(ITargetPlatform* Platform, TArray<FName>& RequiredFormats)
		{
			Platform->GetAllTargetedShaderFormats(RequiredFormats);
		}
#endif
	};

	virtual const TArray<const IAudioFormat*>& GetAudioFormats() override
	{
		return GetFormatsWithHints<IAudioFormat, IAudioFormatModule, FAudioHintHelper>();
	}

	virtual const IAudioFormat* FindAudioFormat(FName Name) override
	{
		const TArray<const IAudioFormat*>& AudioFormats = GetAudioFormats();

		for (int32 Index = 0; Index < AudioFormats.Num(); Index++)
		{
			TArray<FName> Formats;

			AudioFormats[Index]->GetSupportedFormats(Formats);

			for (int32 FormatIndex = 0; FormatIndex < Formats.Num(); FormatIndex++)
			{
				if (Formats[FormatIndex] == Name)
				{
					return AudioFormats[Index];
				}
			}
		}

		return nullptr;
	}

	virtual const ITextureFormat* FindTextureFormat(FName Name) override
	{
		return TextureFormatManager->FindTextureFormat(Name);
	}

	virtual const TArray<const IShaderFormat*>& GetShaderFormats() override
	{
		//if (!AllowShaderCompiling())
		//{
		//	static TArray<const IShaderFormat*> Empty;
		//	return Empty;
		//}

		return GetFormatsWithHints<IShaderFormat, IShaderFormatModule, FShaderHintHelper>();
	}

	virtual const IShaderFormat* FindShaderFormat(FName Name) override
	{
		return ::FindShaderFormat(Name, GetShaderFormats());
	}

	virtual uint32 ShaderFormatVersion(FName Name) override
	{
		static bool bInitialized = false;
		static FRWLock InitializedLock;
		static TMap<FName, uint32> FormatVersionCache;

		// Check if result is already initialized with lightweight read-only lock
		{
			FRWScopeLock InitializedLockGuard(InitializedLock, FRWScopeLockType::SLT_ReadOnly);
			if (bInitialized && !bForceCacheUpdate)
			{
				return FindShaderFormatVersion(FormatVersionCache, Name);
			}
		}

		// Initialize the result now
		{
			FRWScopeLock InitializedLockGuard(InitializedLock, FRWScopeLockType::SLT_Write);
			// Check if another thread initialized the result before this thread received the write lock
			if (!bInitialized || bForceCacheUpdate)
			{
				InitializeShaderFormatVersions(FormatVersionCache);
				bInitialized = true;
			}
		}

		return FindShaderFormatVersion(FormatVersionCache, Name);
	}

	virtual const TArray<const IPhysXCooking*>& GetPhysXCooking() override
	{
		static TArray<const IPhysXCooking*> Results;

		return Results;
	}

	virtual const IPhysXCooking* FindPhysXCooking(FName Name) override
	{
		return nullptr;
	}

	virtual void SetupEnvironmentVariables(TArray<FString>& EnvVarNames, const TArray<FString>& EnvVarValues) override
	{
		checkf(EnvVarNames.Num() == EnvVarValues.Num(), TEXT("EnvVarNames and EnvVarValues are expected to have the same number of elements."));
		for (int i = 0; i < EnvVarNames.Num(); ++i)
		{
			const FString& EnvVarName = EnvVarNames[i];
			const FString& EnvVarValue = EnvVarValues[i];
			UE_LOGF(LogTargetPlatformManager, Verbose, "Setting variable '%ls' to '%ls'.", *EnvVarName, *EnvVarValue);
			FPlatformMisc::SetEnvironmentVar(*EnvVarName, *EnvVarValue);
		}
	}

protected:

	bool InitializeSinglePlatform(FName PlatformName, const FString& AutoSDKPath)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTargetPlatformManagerModule::InitializeSinglePlatform);

		// try the incoming name as a module name, or as a platform name
		FName PlatformModuleName = PlatformName;
		//@todo Custom TargetPlatforms
		FName PlatformControlsModuleName = *(PlatformName.ToString() + TEXT("TargetPlatformControls"));;
		FName PlatformSettingsModuleName = *(PlatformName.ToString() + TEXT("TargetPlatformSettings"));;

		ITargetPlatformModule* Module = nullptr;
		ITargetPlatformControlsModule* ModuleControls = nullptr;
		ITargetPlatformSettingsModule* ModuleSettings = nullptr;

		FModuleManager& ModuleManager = FModuleManager::Get();

		if (!ModuleManager.ModuleExists(*PlatformModuleName.ToString()))
		{
			PlatformModuleName = *(PlatformName.ToString() + TEXT("TargetPlatform"));
		}

		if (ModuleManager.ModuleExists(*PlatformModuleName.ToString()))
		{
			Module = FModuleManager::LoadModulePtr<ITargetPlatformModule>(PlatformModuleName);
		}

		if (ModuleManager.ModuleExists(*PlatformSettingsModuleName.ToString()))
		{
			ModuleSettings = FModuleManager::LoadModulePtr<ITargetPlatformSettingsModule>(PlatformSettingsModuleName);
		}

		bool bIsRunningPlatform = false;
		if(ModuleSettings != nullptr)
		{
			TArray<ITargetPlatformSettings*> TargetPlatformSettings = ModuleSettings->GetTargetPlatformSettings();
			for (ITargetPlatformSettings* Platform : TargetPlatformSettings)
			{
				// one running platform
				if(Platform->IsRunningPlatform())
				{
					checkf(!bIsRunningPlatform, TEXT("Cannot have more than one Running Platform"));
					bIsRunningPlatform = true;
				}
				PlatformSettings.Add(Platform);
				if (Module != nullptr)
				{
					Module->PlatformSettings.Add(Platform);
				}
			}
		}

		if (Module == nullptr)
		{
			if (ModuleManager.ModuleExists(*PlatformModuleName.ToString()))
			{
				// Retry module load with error logging enabled
				EModuleLoadResult Result;
				Module = static_cast<ITargetPlatformModule*>(ModuleManager.LoadModuleWithFailureReason(PlatformModuleName, Result, ELoadModuleFlags::LogFailures));
				UE_CLOGFMT(!Module, LogTargetPlatformManager, Warning,
					"Failed to load module '{PlatformModuleName}' for platform {PlatformName} (Reason={Reason}).",
					PlatformModuleName, PlatformName.ToString(), LexToString(Result));
			}
			else
			{
				UE_LOGFMT(LogTargetPlatformManager, Log,
					"Failed to load module '{PlatformModuleName}' for platform {PlatformName} (Reason={Reason}).",
					PlatformModuleName, PlatformName.ToString(), LexToString(EModuleLoadResult::FileNotFound));
			}
		}

		// original logic for module loading here
		if (Module)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FTargetPlatformManagerModule::InitializeSinglePlatform Loading);
			// If AutoSDK is enabled, then we skip the check if AutoSDKPath of this platform is Empty.
			// If AutoSDK is disabled for whatever reason, then we do not skip and base the platform validity on UBT.
			// If the platform is not in the DataDrivenPlatformInfoRegistry, it's a custom platform module
			// that doesn't participate in the standard SDK management system, so skip SDK validation
			bool bSkipSDKCheck = !FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos().Contains(PlatformName);
			if (bSkipSDKCheck)
			{
				UE_LOGF(LogTargetPlatformManager, Log, "Platform '%ls' not in DataDrivenPlatformInfoRegistry, skipping SDK validation", *PlatformName.ToString());
			}
			FName PlatformSDKNameToCheck = PlatformName;
			// would like to move this check to GetActiveTargetPlatforms, but too many things cache this result
			// this setup will become faster after TTP 341897 is complete.
		RETRY_SETUPANDVALIDATE:
			// This is done here in case autosdk setup failed because shared files are in use by another child cooker
			if (IsAutoSDKsEnabled())
			{
				bSkipSDKCheck = bSkipSDKCheck || (AutoSDKPath == TEXT(""));
				PlatformSDKNameToCheck = FName(*AutoSDKPath);
				if (!bSkipSDKCheck)
				{
					SetupAutoSDK(AutoSDKPath);
				}
			}

			if (bSkipSDKCheck || bIsRunningPlatform || ValidateSDK(PlatformSDKNameToCheck))
			{
				if (ModuleManager.ModuleExists(*PlatformControlsModuleName.ToString()))
				{
					ModuleControls = FModuleManager::LoadModulePtr<ITargetPlatformControlsModule>(PlatformControlsModuleName);
				}

				if (ModuleControls != nullptr)
				{
					TArray<ITargetPlatformControls*> TargetPlatformControls = ModuleControls->GetTargetPlatformControls(PlatformSettingsModuleName);
					for (ITargetPlatformControls* Platform : TargetPlatformControls)
					{
						PlatformControls.Add(Platform);
						Module->PlatformControls.Add(Platform);
					}
				}

				TArray<ITargetPlatform*> TargetPlatforms = Module->GetTargetPlatforms();
				for (ITargetPlatform* Platform : TargetPlatforms)
				{
					UE_LOGF(LogTargetPlatformManager, Display, "Loaded TargetPlatform '%ls'", *Platform->PlatformName());
					Platforms.Add(Platform);
					PlatformsByName.Add(FName(Platform->PlatformName()), Platform);
				}

				// only success path
				return true;
			}
			else
			{
				// this hack is here because if you try and setup and validate autosdk some times it will fail because shared files are in use by another child cooker
				static bool bIsChildCooker = FParse::Param(FCommandLine::Get(), TEXT("cookchild"));
				if (bIsChildCooker)
				{
					static int Counter = 0;
					++Counter;
					if (Counter < 10)
					{
						goto RETRY_SETUPANDVALIDATE;
					}
				}
				UE_LOGF(LogTargetPlatformManager, Display, "Failed to SetupSDK for platform '%ls'", *PlatformName.ToString());
			}
		}

		return false;
	}

	/** Discovers the available target platforms. */
	void DiscoverAvailablePlatforms()
	{
		DECLARE_SCOPE_CYCLE_COUNTER(TEXT("FTargetPlatformManagerModule::DiscoverAvailablePlatforms"), STAT_FTargetPlatformManagerModule_DiscoverAvailablePlatforms, STATGROUP_TargetPlatform);

		Platforms.Empty(Platforms.Num());
		PlatformsByName.Empty(PlatformsByName.Num());

#if !IS_MONOLITHIC
		// Find all module subdirectories and add them so we can load dependent modules for target platform modules
		// We may not be able to restrict this to subdirectories found in FPlatformInfo because we could have a subdirectory
		// that is not one of these platforms. Imagine a "Sega" shared directory for the "Genesis" and "Dreamcast" platforms
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FTargetPlatformManagerModule::DiscoverAvailablePlatforms FindFilesRecursive);

			TArray<FString> ModuleSubdirs;
			IFileManager::Get().FindFilesRecursive(ModuleSubdirs, *FPlatformProcess::GetModulesDirectory(), TEXT("*"), false, true);
			for (const FString& ModuleSubdir : ModuleSubdirs)
			{
				FModuleManager::Get().AddBinariesDirectory(*ModuleSubdir, false);
			}
		}
#endif

#if WITH_EDITOR
		// Get the platform we are previewing if GMaxRHIShaderPlatform is a preview SP
		FName PlatformNamePreview = NAME_None;
		if (IsRunningGame())
		{
			if (FDataDrivenShaderPlatformInfo::GetIsPreviewPlatform(GMaxRHIShaderPlatform))
			{
				PlatformNamePreview = FDataDrivenShaderPlatformInfo::GetPlatformName(GMaxRHIShaderPlatform);
			}
		}
		
#endif

		// find a set of valid target platform names (the platform DataDrivenPlatformInfo.ini file was found indicates support for the platform 
		// exists on disk, so the TP is expected to work)
		FScopedSlowTask SlowTask((float)FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos().Num());
		for (auto Pair : FDataDrivenPlatformInfoRegistry::GetAllPlatformInfos())
		{
			FName PlatformName = Pair.Key;

			TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(*PlatformName.ToString());

			const FDataDrivenPlatformInfo& Info = Pair.Value;

			SlowTask.EnterProgressFrame(1);

#if WITH_EDITOR
			// if we have the editor and we are using -game (but not with -platformrhi)
			// only need to instantiate the current platform if we are not using OverrideSP command
			// If we are using OverrideSP command that means our SP is a preview SP 
			// Because of that we must also load the TPS of the Platform we are previewing.
			if (IsRunningGame())
			{
				static bool bUsingPlatformRHI = FString(FCommandLine::Get()).Contains(TEXT("-platformrhi="));
				if (!bUsingPlatformRHI && PlatformName != FPlatformProperties::IniPlatformName() && 
					(PlatformName != PlatformNamePreview))
				{
					continue;
				}
			}
#endif

			if (Info.bEnabledForUse)
			{
				InitializeSinglePlatform(PlatformName, Info.AutoSDKPath);
			}
		}

		TArray<FString> CustomTargetPlatformModules;
		GConfig->GetArray(TEXT("CustomTargetPlatforms"), TEXT("ModuleName"), CustomTargetPlatformModules, GEditorIni);
		for (const FString& ModuleName : CustomTargetPlatformModules)
		{
			InitializeSinglePlatform(*ModuleName, TEXT(""));
		}


		if (!Platforms.Num())
		{
			UE_LOGF(LogTargetPlatformManager, Error, "No target platforms found!");
		}
	}

	bool SetupEnvironmentFromAutoSDK(const FString& AutoSDKPath)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTargetPlatformManagerModule::SetupEnvironmentFromAutoSDK);

		if (!IsAutoSDKsEnabled())
		{
			return false;
		}

		// Invoke UBT to perform SDK switching, or detect that a proper manual SDK is already setup.				
#if PLATFORM_WINDOWS
		FString HostPlatform(TEXT("HostWin64"));
#else
		FString HostPlatform = FString::Printf(TEXT("Host%hs"), FPlatformProperties::IniPlatformName());
#endif		

		FString TargetSDKRoot = GetAutoSDKPlatformDir(*AutoSDKPath);
		FString SDKInstallManifestFilePath = TargetSDKRoot / UE::AutoSDK::SDKInstallManifestFileName;

		TArray<FString> FileLines;
		// If we are using a manual install, then it is valid for there to be no OutputEnvVars file, so 
		// check for ManualINstall in the CurrentlyInstalled file
		if (FFileHelper::LoadFileToStringArray(FileLines, *SDKInstallManifestFilePath))
		{
			if (FileLines.Num() != 2 && FileLines.Num() != 3)
			{
				UE_LOGF(LogTargetPlatformManager, Warning, "Malformed install manifest file for Platform %ls", *AutoSDKPath);
				return false;
			}

			static const FString ManualSDKString(TEXT("ManualSDK"));
			if (FileLines[1].Compare(ManualSDKString, ESearchCase::IgnoreCase) == 0)
			{
				UE_LOGF(LogTargetPlatformManager, Verbose, "Platform %ls has manual sdk install", *AutoSDKPath);
				return true;
			}
		}
		else
		{
			UE_LOGF(LogTargetPlatformManager, Log, "Install manifest file for Platform %ls not found.  Platform not set up.", *AutoSDKPath);
			return false;			
		}

		static const FString SDKEnvironmentVarsFile(TEXT("OutputEnvVars.txt"));
		FString EnvVarFileName = FPaths::Combine(*TargetSDKRoot, *SDKEnvironmentVarsFile);

		if (!SetupEnvironmentFromEnvVarFile(EnvVarFileName))
		{
			UE_LOGF(LogTargetPlatformManager, Warning, "OutputEnvVars.txt not found for platform: '%ls'", *AutoSDKPath);
			return false;
		}

		UE_LOGF(LogTargetPlatformManager, Verbose, "Platform %ls has auto sdk install", *AutoSDKPath);
		return true;
	}

	void SetupAutoSDK(const FString& AutoSDKPath)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTargetPlatformManagerModule::SetupAndValidateAutoSDK);
		bool bValidSDK = false;
		if (AutoSDKPath.Len() > 0)
		{		
			FName AutoSDKName(*AutoSDKPath);

			// cache result of the last setup attempt to avoid calling UBT all the time.
			bool* bPreviousSetupSuccessful = PlatformsSetup.Find(AutoSDKName);
			if (!bPreviousSetupSuccessful)
			{
				bValidSDK = SetupEnvironmentFromAutoSDK(AutoSDKPath);
				PlatformsSetup.Add(AutoSDKName, bValidSDK);
			}
		}
	}

	bool ValidateSDK(FName PlatformName)
	{
		bool bValidSDK = false;
		bool* SetupSuccessful = PlatformsSetup.Find(PlatformName);
		if (SetupSuccessful)
		{
			bValidSDK = *SetupSuccessful;
		}
		return bValidSDK;
	}

	bool SetupEnvironmentFromManualSDK(const FString& EnvVarFileName)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTargetPlatformManagerModule::SetupEnvironmentFromManualSDK);

		UE_LOGF(LogTargetPlatformManager, Verbose, "Reading the manifest for auto-selected manual sdks" );
		bool bResult = SetupEnvironmentFromEnvVarFile(EnvVarFileName);
		IFileManager::Get().Delete(*EnvVarFileName);
		return bResult;
	}

	bool SetupEnvironmentFromEnvVarFile( const FString& EnvVarFileName )
	{
		// If we are using a manual install, then it is valid for there to be no OutputEnvVars file.
		TUniquePtr<FArchive> EnvVarFile(IFileManager::Get().CreateFileReader(*EnvVarFileName));
		if (EnvVarFile)
		{
			TArray<FString> FileLines;
			{
				int64 FileSize = EnvVarFile->TotalSize();
				int64 MemSize = FileSize + 1;
				void* FileMem = FMemory::Malloc(MemSize);
				FMemory::Memset(FileMem, 0, MemSize);

				EnvVarFile->Serialize(FileMem, FileSize);

				FString FileAsString(ANSI_TO_TCHAR(FileMem));
				FileAsString.ParseIntoArrayLines(FileLines);

				FMemory::Free(FileMem);
				EnvVarFile->Close();
			}

			TArray<FString> PathAdds;
			TArray<FString> PathRemoves;
			TArray<FString> EnvVarNames;
			TArray<FString> EnvVarValues;

			const FString VariableSplit(TEXT("="));
			for (int32 i = 0; i < FileLines.Num(); ++i)
			{				
				const FString& VariableString = FileLines[i];

				FString Left;
				FString Right;
				VariableString.Split(VariableSplit, &Left, &Right);
				
				if (Left.Compare(TEXT("strippath"), ESearchCase::IgnoreCase) == 0)
				{
					PathRemoves.Add(Right);
				}
				else if (Left.Compare(TEXT("addpath"), ESearchCase::IgnoreCase) == 0)
				{
					PathAdds.Add(Right);
				}
				else
				{
					// convenience for setup.bat writers.  Trim any accidental whitespace from var names/values.
					EnvVarNames.Add(Left.TrimStartAndEnd());
					EnvVarValues.Add(Right.TrimStartAndEnd());
				}
			}

			// don't actually set anything until we successfully validate and read all values in.
			// we don't want to set a few vars, return a failure, and then have a platform try to
			// build against a manually installed SDK with half-set env vars.
			SetupEnvironmentVariables(EnvVarNames, EnvVarValues);


			FString OrigPathVar = FPlatformMisc::GetEnvironmentVariable(TEXT("PATH"));

			// actually perform the PATH stripping / adding.
			const TCHAR* PathDelimiter = FPlatformMisc::GetPathVarDelimiter();
			TArray<FString> PathVars;
			OrigPathVar.ParseIntoArray(PathVars, PathDelimiter, true);

			TArray<FString> ModifiedPathVars;
			ModifiedPathVars = PathVars;

			// perform removes first, in case they overlap with any adds.
			for (int32 PathRemoveIndex = 0; PathRemoveIndex < PathRemoves.Num(); ++PathRemoveIndex)
			{
				const FString& PathRemove = PathRemoves[PathRemoveIndex];
				for (int32 PathVarIndex = 0; PathVarIndex < PathVars.Num(); ++PathVarIndex)
				{
					const FString& PathVar = PathVars[PathVarIndex];
					if (PathVar.Find(PathRemove, ESearchCase::IgnoreCase) >= 0)
					{
						UE_LOGF(LogTargetPlatformManager, Verbose, "Removing Path: '%ls'", *PathVar);
						ModifiedPathVars.Remove(PathVar);
					}
				}
			}

			// remove all the of ADDs so that if this function is executed multiple times, the paths will be guarateed to be in the same order after each run.
			// If we did not do this, a 'remove' that matched some, but not all, of our 'adds' would cause the order to change.
			for (int32 PathAddIndex = 0; PathAddIndex < PathAdds.Num(); ++PathAddIndex)			
			{
				const FString& PathAdd = PathAdds[PathAddIndex];				
				for (int32 PathVarIndex = 0; PathVarIndex < PathVars.Num(); ++PathVarIndex)
				{
					const FString& PathVar = PathVars[PathVarIndex];
					if (PathVar.Find(PathAdd, ESearchCase::IgnoreCase) >= 0)
					{
						UE_LOGF(LogTargetPlatformManager, Verbose, "Removing Path: '%ls'", *PathVar);
						ModifiedPathVars.Remove(PathVar);
					}
				}
			}

			// perform adds, but don't add duplicates
			for (int32 PathAddIndex = 0; PathAddIndex < PathAdds.Num(); ++PathAddIndex)
			{
				const FString& PathAdd = PathAdds[PathAddIndex];
				if (!ModifiedPathVars.Contains(PathAdd))
				{
					UE_LOGF(LogTargetPlatformManager, Verbose, "Adding Path: '%ls'", *PathAdd);					
					ModifiedPathVars.Add(PathAdd);
				}
			}

			FString ModifiedPath = FString::Join(ModifiedPathVars, PathDelimiter);
			FPlatformMisc::SetEnvironmentVar(TEXT("PATH"), *ModifiedPath);			
			return true;
		}
		else
		{
			return false;
		}
	}

	bool UpdateAfterSDKInstall(FName PlatformName)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTargetPlatformManagerModule::UpdateAfterSDKInstall);

		const FDataDrivenPlatformInfo& Info = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(PlatformName);


		FName AutoSDKName = NAME_None;
		FString AutoSDKPath;
		if(IsAutoSDKsEnabled())
		{
			AutoSDKPath = Info.AutoSDKPath;
			AutoSDKName = *AutoSDKPath;
			if (AutoSDKName != NAME_None)
			{
				// make sure we can re-do the AutoSDK setup
				PlatformsSetup.Remove(AutoSDKName);
			}
		}
		else
		{
			// Must call UBT again to update platform status if AutoSDK is off.
			LaunchUBTValidatePlatformsProc(PlatformName);
		}
		
		SDKSetup(PlatformName);

		// note: this assumes, along with other Turnkey code, that there is a TargetPlatform named with the IniPlatformName
		ITargetPlatform* TargetPlatform = FindTargetPlatform(PlatformName);

		bool bTPInitialized = false;
		// if we didn't have a TP before, discover it now, it will do everything we need)
		if (TargetPlatform == nullptr)
		{
			// create the TP(s) that weren't around before due to a bad SDK
			if (Info.bEnabledForUse)
			{
				bTPInitialized = InitializeSinglePlatform(PlatformName, Info.AutoSDKPath);
			}
		}
		else
		{
			if (IsAutoSDKsEnabled())
			{
				if (!AutoSDKPath.IsEmpty())
				{
					// setup AutoSDK, and then re-initialize the TP
					SetupAutoSDK(AutoSDKPath);
				}
			}
			bTPInitialized = TargetPlatform->InitializeHostPlatform();
		}

#if UE_WITH_TURNKEY_SUPPORT
		if (bTPInitialized)
		{
			ITurnkeySupportModule::Get().UpdateSdkInfo();
		}

		ITurnkeySupportModule::Get().ClearDeviceStatus(PlatformName);
#endif
		Invalidate();

		return bTPInitialized;
	}

private:

	void ModulesChangesCallback(FName ModuleName, EModuleChangeReason ReasonForChange)
	{
		if (bIsInDelegateCall)
		{
			return;
		}
		
		if (!bIgnoreFirstDelegateCall && ModuleName.ToString().Contains(TEXT("TargetPlatform")) && !ModuleName.ToString().Contains(TEXT("ProjectTargetPlatformEditor")))
		{
			bIsInDelegateCall = true;
			Invalidate();
			bIsInDelegateCall = false;
		}
		bIgnoreFirstDelegateCall = false;
	}

	bool InitializeActiveTargetPlatforms(TArray<ITargetPlatform*>& OutResults)
	{
		bHasInitErrors = false; // If we had errors before, reset the flag and see later in this function if there are errors.
		InitErrorMessages.Empty();

		OutResults.Empty(OutResults.Num());

		const TArray<ITargetPlatform*>& TargetPlatforms = GetTargetPlatforms();

		FString PlatformStr;

		if (FParse::Value(FCommandLine::Get(), TEXT("TARGETPLATFORM="), PlatformStr))
		{
			if (PlatformStr == TEXT("None"))
			{
			}
			else if (PlatformStr == TEXT("All"))
			{
				OutResults = TargetPlatforms;
			}
			else
			{
				TArray<FString> PlatformNames;

				PlatformStr.ParseIntoArray(PlatformNames, TEXT("+"), true);

				for (int32 Index = 0; Index < TargetPlatforms.Num(); Index++)
				{
					if (PlatformNames.Contains(TargetPlatforms[Index]->PlatformName()))
					{
						OutResults.Add(TargetPlatforms[Index]);
					}
				}

				if (OutResults.Num() == 0)
				{
					// An invalid platform was specified...
					// Inform the user.
					TStringBuilder<1024> AvailablePlatforms;
					for (int32 Index = 0; Index < TargetPlatforms.Num(); Index++)
					{
						if (Index > 0)
						{
							AvailablePlatforms << TEXT(", ");
						}
						AvailablePlatforms << TargetPlatforms[Index]->PlatformName();
					}
					bHasInitErrors = true;
					InitErrorMessages.Appendf(TEXT("Invalid target platform specified (%s). Available = { %s } "), *PlatformStr, *AvailablePlatforms);
					UE_LOGF(LogTargetPlatformManager, Error, "Invalid target platform specified (%ls). Available = { %ls } ", *PlatformStr, *AvailablePlatforms);
					return false;
				}
			}
		}
		else
		{
			// if there is no argument, use the current platform and only build formats that are actually needed to run.
			bRestrictFormatsToRuntimeOnly = true;

			for (int32 Index = 0; Index < TargetPlatforms.Num(); Index++)
			{
				if (TargetPlatforms[Index]->IsRunningPlatform())
				{
					OutResults.Add(TargetPlatforms[Index]);
				}
			}
		}

		if (!OutResults.Num())
		{
			UE_LOGF(LogTargetPlatformManager, Display, "Not building assets for any platform.");
		}
		else
		{
			for (int32 Index = 0; Index < OutResults.Num(); Index++)
			{
				UE_LOGF(LogTargetPlatformManager, Display, "Building Assets For %ls", *OutResults[Index]->PlatformName());
			}
		}

		return true;
	}

	template<typename FormatType, typename FormatModuleType, typename HelperType>
	void InitializeFormatsWithHints(TArray<const FormatType*>& OutResults)
	{
		OutResults.Empty(OutResults.Num());

		// the functions for dealing with hints are only defined with Engine				
#if WITH_ENGINE 
		TArray<FName> SupportedFormatsByHints;
		TSet<FName> RequiredFormats;
		// gather the hinted formats, and the needed formats for all the active targetplatforms
		TArray<ITargetPlatform*> TargetPlatforms = GetTargetPlatforms();
		for (ITargetPlatform* Platform : TargetPlatforms)
		{
			TArray<FName> FormatHints;
			HelperType::GetHintedModules(Platform, FormatHints);
			for (FName HintedModuleName : FormatHints)
			{
				FormatModuleType* Module = FModuleManager::Get().LoadModulePtr<FormatModuleType>(HintedModuleName);
				if (Module != nullptr)
				{
					FormatType* Format = HelperType::GetFormatFromModule(Module);
					if (Format && !OutResults.Contains(Format))
					{
						// remember the module
						OutResults.Add(Format);
						// remember its formats
						Format->GetSupportedFormats(SupportedFormatsByHints);
					}
				}
			}

			// remember the formats the TP needs
			TArray<FName> PlatformRequiredFormats;
			HelperType::GetRequiredFormats(Platform, PlatformRequiredFormats);
			RequiredFormats.Append(PlatformRequiredFormats);
		}

		// make sure every required format was found above
		bool bFoundAllFormats = true;
		for (FName Format : RequiredFormats)
		{
			if (!SupportedFormatsByHints.Contains(Format))
			{
				UE_LOGF(LogTargetPlatformManager, Log, "Unable to find %ls format %ls from hinted modules, loading all potential format modules to find it", HelperType::GetFormatDesc(), *Format.ToString());
				bFoundAllFormats = false;
				break;
			}
		}

		// if we found all the formats from the hints, we are done, and Results is filled out
		if (bFoundAllFormats)
		{
			return;
		}
#endif

		// if the hints weren't enough to find everything, then load all modules		
		TArray<FName> Modules;
		FModuleManager::Get().FindModules(HelperType::GetAllModuleWildcard(), Modules);

		if (!Modules.Num())
		{
#if IS_PROGRAM
			// Only log this information to file, but don't report this as an error for standalone programs.
			//   UnrealPak for instance loads TPM to request shader formats only, but doesn't care about audio formats.
			//   The only format UnrealPak is built against is MetalShaderFormat to check whether native shader libs are supported.
			UE_LOGF(LogTargetPlatformManager, Log, "No target %ls formats found!", HelperType::GetFormatDesc());
#else
			UE_LOGF(LogTargetPlatformManager, Error, "No target %ls formats found!", HelperType::GetFormatDesc());
#endif
		}

		for (int32 Index = 0; Index < Modules.Num(); Index++)
		{
			FormatModuleType* Module = FModuleManager::LoadModulePtr<FormatModuleType>(Modules[Index]);
			if (Module)
			{
				FormatType* Format = HelperType::GetFormatFromModule(Module);
				UE_LOGF(LogTargetPlatformManager, Log, "Loaded format module %ls", *Modules[Index].ToString());
				if (Format != nullptr)
				{
					TArray<FName> Formats;
					Format->GetSupportedFormats(Formats);
					for (FName Name : Formats)
					{
						UE_LOGF(LogTargetPlatformManager, Log, "  %ls", *Name.ToString());
					}

					OutResults.AddUnique(Format);
				}
			}
		}
	}

	void InitializeShaderFormatVersions(TMap<FName, uint32>& OutFormatVersionCache)
	{
		OutFormatVersionCache.Reset();

		for (const IShaderFormat* SF : GetShaderFormats())
		{
			TArray<FName> Formats;
			SF->GetSupportedFormats(Formats);
			for (FName FormatName : Formats)
			{
				OutFormatVersionCache.FindOrAdd(FormatName, SF->GetVersion(FormatName));
			}
		}
	}

	uint32 FindShaderFormatVersion(const TMap<FName, uint32>& FormatVersionCache, FName Name) const
	{
		const uint32* Result = FormatVersionCache.Find(Name);
		if (!Result)
		{
			UE_LOGF(LogTargetPlatformManager, Fatal, "ShaderFormat not found for %ls!  Dynamically loaded shader formats require invalidation of FormatVersionCache.", *Name.ToString());
			return INDEX_NONE;
		}
		return *Result;
	}

	static FString SDKStatusMessage;
	static void OnStatusOutput(FString Message)
	{
		SDKStatusMessage += Message;
	}

	FString InitErrorMessages;

	// Delegate used to notify users of returned ITargetPlatform* pointers when those pointers are destructed due to a call to Invalidate
	FOnTargetPlatformsInvalidated OnTargetPlatformsInvalidated;

	// If true we should build formats that are actually required for use by the runtime. 
	// This happens for an ordinary editor run and more specifically whenever there is no
	// TargetPlatform= on the command line.
	bool bRestrictFormatsToRuntimeOnly;

	bool bIsInitialized = false;

	// Flag to force reinitialization of all cached data. This is needed to have up-to-date caches
	// in case of a module reload of a TargetPlatform-Module.
	bool bForceCacheUpdate;

	// Flag to indicate that there were errors during initialization
	bool bHasInitErrors;

	// Flag to avoid redundant reloads
	bool bIgnoreFirstDelegateCall;
	
	// Flag to avoid recursive reloads
	bool bIsInDelegateCall = false;
	
	// Holds the list of discovered platforms.
	TArray<ITargetPlatform*> Platforms;
	TArray<ITargetPlatformControls*> PlatformControls;
	TArray<ITargetPlatformSettings*> PlatformSettings;

	// Map for fast lookup of platforms by name.
	TMap<FName, ITargetPlatform*> PlatformsByName;

	// External module that texture format operations are forwarded to
	ITextureFormatManagerModule* TextureFormatManager;

	// holds the list of Platforms that have attempted setup.
	// When AutoSDK is enabled this holds the AutoSDKName because multiple platform can use the same AutoSDKName
	TMap<FName, bool> PlatformsSetup;
};


FString FTargetPlatformManagerModule::SDKStatusMessage = TEXT("");


IMPLEMENT_MODULE(FTargetPlatformManagerModule, TargetPlatform);
