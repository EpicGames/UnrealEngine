// Copyright Epic Games, Inc. All Rights Reserved.

#include "Common/TargetPlatformSettingsBase.h"
#include "AnalyticsEventAttribute.h"
#include "HAL/IConsoleManager.h"

int32 GFallbackASTCHDRProfile = 0;
static FAutoConsoleVariableRef CVarFallbackASTCHDRProfile(
	TEXT("cook.FallbackForASTCHDRProfile"),
	GFallbackASTCHDRProfile,
	TEXT("What Fallback Format we use if ASTC HDR Profile is not supported.")
	TEXT("0 = Fallback to ASTC LDR")
	TEXT("1 = Fallback to RGBA16F"),
	ECVF_Default | ECVF_ReadOnly
);

bool FTargetPlatformSettingsBase::UsesForwardShading() const
{
	static IConsoleVariable* CVarForwardShading = IConsoleManager::Get().FindConsoleVariable(TEXT("r.ForwardShading"));
	return CVarForwardShading ? (CVarForwardShading->GetInt() != 0) : false;
}

bool FTargetPlatformSettingsBase::UsesDBuffer() const
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DBuffer"));
	return CVar ? (CVar->GetInt() != 0) : false;
}

bool FTargetPlatformSettingsBase::UsesBasePassVelocity() const
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.VelocityOutputPass"));
	return CVar ? (CVar->GetInt() == 1) : false;
}

bool FTargetPlatformSettingsBase::VelocityEncodeDepth() const
{
	return true;
}

bool FTargetPlatformSettingsBase::UsesSelectiveBasePassOutputs() const
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.SelectiveBasePassOutputs"));
	return CVar ? (CVar->GetInt() != 0) : false;
}

bool FTargetPlatformSettingsBase::ShouldStripNaniteFallbackMeshes() const
{
	return false;
}

bool FTargetPlatformSettingsBase::UsesDistanceFields() const
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.DistanceFields"));
	return CVar ? (CVar->GetInt() != 0) : false;
}

bool FTargetPlatformSettingsBase::UsesRayTracing() const
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.RayTracing"));
	return CVar ? (CVar->GetInt() != 0) : false;
}

ERayTracingRuntimeMode FTargetPlatformSettingsBase::GetRayTracingMode() const
{
	return ERayTracingRuntimeMode::Disabled;
}

ERayTracingRuntimeMode FTargetPlatformSettingsBase::ParseRuntimeRayTracingMode(const TCHAR* SectionName) const
{
	FString ModeString;
	GetConfigSystem()->GetString(SectionName, TEXT("RayTracingMode"), ModeString, GEngineIni);
	if (ModeString.Equals(TEXT("Inline"), ESearchCase::IgnoreCase))
	{
		return ERayTracingRuntimeMode::Inline;
	}
	else if (ModeString.Equals(TEXT("Full"), ESearchCase::IgnoreCase))
	{
		return ERayTracingRuntimeMode::Full;
	}
	return ERayTracingRuntimeMode::Disabled;
}

uint32 FTargetPlatformSettingsBase::GetSupportedHardwareMask() const
{
	return 0;
}

EOfflineBVHMode FTargetPlatformSettingsBase::GetStaticMeshOfflineBVHMode() const
{
	return EOfflineBVHMode::Disabled;
}

bool FTargetPlatformSettingsBase::GetStaticMeshOfflineBVHCompression() const
{
	return false;
}

EOfflineBVHMode FTargetPlatformSettingsBase::GetSkeletalMeshOfflineBVHMode() const
{
	return EOfflineBVHMode::Disabled;
}

bool FTargetPlatformSettingsBase::ForcesSimpleSkyDiffuse() const
{
	return false;
}

float FTargetPlatformSettingsBase::GetDownSampleMeshDistanceFieldDivider() const
{
	return 1.0f;
}

int32 FTargetPlatformSettingsBase::GetHeightFogModeForOpaque() const
{
	// Don't override the project setting by default
	// Platforms wish to support override need to implement the logic in their own target platform classes
	return 0;
}

bool FTargetPlatformSettingsBase::UsesMobileDBuffer() const
{
	static IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("r.Mobile.DBuffer"));
	return CVar ? (CVar->GetInt() != 0) : false;
}

bool FTargetPlatformSettingsBase::UsesASTCHDR() const
{
	static IConsoleVariable* CVarASTCHDR = IConsoleManager::Get().FindConsoleVariable(TEXT("cook.AllowASTCHDRProfile"));
	return (CVarASTCHDR ? (CVarASTCHDR->GetInt() != 0) : false);
}

FName FTargetPlatformSettingsBase::GetFallbackASTCHDR() const
{
	static const FName NameRGBA16F(TEXT("RGBA16F"));
	static const FName NameASTC_RGB_LDR(TEXT("ASTC_RGB"));

	switch (GFallbackASTCHDRProfile)
	{
	case 0:
		return NameASTC_RGB_LDR;
	case 1:
		return NameRGBA16F;
	}
	return NameRGBA16F;
}

void FTargetPlatformSettingsBase::GetRayTracingShaderFormats(TArray<FName>& OutFormats) const
{
	if (UsesRayTracing())
	{
		GetAllTargetedShaderFormats(OutFormats);
	}
}

void FTargetPlatformSettingsBase::GetPossibleArchitectures(TArray<FString>& OutArchitectures) const
{
}

FString FTargetPlatformSettingsBase::GetHostArchitecture() const
{
	return FString();
}
