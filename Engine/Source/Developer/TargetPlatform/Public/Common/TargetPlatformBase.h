// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreGlobals.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "HAL/PlatformMisc.h"
#include "Interfaces/ITargetPlatform.h"
#include "Internationalization/Text.h"
#include "Misc/AssertionMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "PlatformInfo.h"
#include "Templates/PimplPtr.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"

class IDeviceManagerCustomPlatformWidgetCreator;
class IPlugin;
struct FDataDrivenPlatformInfo;

// Forward declare.
namespace Audio 
{
	class FAudioFormatSettings;
}

/**
 * Base class for target platforms.
 */
class UE_DEPRECATED(5.5, "FTargetPlatformBase is deprecated, please use FTargetPlatformControlsBase(SDK required to build) or FTargetPlatformSettingsBase(no SDK required).") FTargetPlatformBase
	: public ITargetPlatform
{
public:

	// ITargetPlatform interface

	virtual bool AddDevice( const FString& DeviceName, bool bDefault ) override
	{
		return false;
	}

	virtual bool AddDevice(const FString& DeviceId, const FString& DeviceUserFriendlyName, const FString& Username, const FString& Password, bool bDefault) override
	{
		return AddDevice(DeviceId, bDefault);
	}

	virtual FText DisplayName() const override
	{
		return PlatformInfo->DisplayName;
	}

	virtual const PlatformInfo::FTargetPlatformInfo& GetTargetPlatformInfo() const override
	{
		return *PlatformInfo;
	}

	virtual const FDataDrivenPlatformInfo& GetPlatformInfo() const override
	{
		return *PlatformInfo->DataDrivenPlatformInfo;
	}

	virtual FConfigCacheIni* GetConfigSystem() const override
	{
		return FConfigCacheIni::ForPlatform(*IniPlatformName());
	}

	TARGETPLATFORM_API virtual bool IsEnabledForPlugin(const IPlugin& Plugin) const override;

	TARGETPLATFORM_API virtual bool UsesForwardShading() const override;

	TARGETPLATFORM_API virtual bool UsesDBuffer() const override;

	UE_DEPRECATED(5.1, "Use IsUsingBasePassVelocity(EShaderPlatform Platform) in renderutils that will uses FShaderPlatformCachedIniValue to retrieve the cvar value per platform.")
	TARGETPLATFORM_API virtual bool UsesBasePassVelocity() const override;

    TARGETPLATFORM_API virtual bool VelocityEncodeDepth() const override;

    TARGETPLATFORM_API virtual bool UsesSelectiveBasePassOutputs() const override;

	TARGETPLATFORM_API virtual bool ShouldStripNaniteFallbackMeshes() const override;
	
	TARGETPLATFORM_API virtual bool UsesDistanceFields() const override;

	TARGETPLATFORM_API virtual bool UsesRayTracing() const override;

	TARGETPLATFORM_API virtual ERayTracingRuntimeMode GetRayTracingMode() const override;

	TARGETPLATFORM_API virtual uint32 GetSupportedHardwareMask() const override;

	TARGETPLATFORM_API virtual EOfflineBVHMode GetStaticMeshOfflineBVHMode() const override;

	TARGETPLATFORM_API virtual bool GetStaticMeshOfflineBVHCompression() const override;

	TARGETPLATFORM_API virtual EOfflineBVHMode GetSkeletalMeshOfflineBVHMode() const override;

	TARGETPLATFORM_API virtual bool ForcesSimpleSkyDiffuse() const override;

	TARGETPLATFORM_API virtual float GetDownSampleMeshDistanceFieldDivider() const override;

	TARGETPLATFORM_API virtual int32 GetHeightFogModeForOpaque() const override;

	TARGETPLATFORM_API virtual bool UsesMobileDBuffer() const override;

	TARGETPLATFORM_API virtual bool UsesASTCHDR() const override;
	
	TARGETPLATFORM_API virtual FName GetFallbackASTCHDR() const override;

	TARGETPLATFORM_API virtual void GetRayTracingShaderFormats(TArray<FName>& OutFormats) const override;

	TARGETPLATFORM_API virtual void GetPlatformSpecificProjectAnalytics( TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray ) const override;

#if WITH_ENGINE
	virtual void GetReflectionCaptureFormats( TArray<FName>& OutFormats ) const override
	{
		OutFormats.Add(FName(TEXT("FullHDR")));
	}

	virtual FName FinalizeVirtualTextureLayerFormat(FName Format) const override
	{
		return Format;
	}

	virtual bool SupportsLQCompressionTextureFormat() const override { return true; };

#endif //WITH_ENGINE

	virtual bool PackageBuild( const FString& InPackgeDirectory ) override
	{
		return true;
	}

	virtual bool CanSupportRemoteShaderCompile() const override
	{
		return true;
	}
	
	virtual void GetShaderCompilerDependencies(TArray<FString>& OutDependencies) const override
	{
	}

	/** Helper method to fill a dependencies array for the shader compiler with absolute paths, passing a relative path to the Engine as the parameter. */
	static void AddDependencySCArrayHelper(TArray<FString>& OutDependencies, const FString& DependencyRelativePath)
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		FString DependencyAbsolutePath = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*(FPaths::EngineDir() / DependencyRelativePath));
		FPaths::NormalizeDirectoryName(DependencyAbsolutePath);
		OutDependencies.AddUnique(DependencyAbsolutePath);
	}

	virtual bool IsSdkInstalled(bool bProjectHasCode, FString& OutDocumentationPath) const override
	{
		return true;
	}

	virtual int32 CheckRequirements(bool bProjectHasCode, EBuildConfiguration Configuration, bool bRequiresAssetNativization, FString& OutTutorialPath, FString& OutDocumentationPath, FText& CustomizedLogMessage) const override
	{
		int32 bReadyToBuild = ETargetPlatformReadyStatus::Ready; // @todo How do we check that the iOS SDK is installed when building from Windows? Is that even possible?
		if (!IsSdkInstalled(bProjectHasCode, OutTutorialPath))
		{
			bReadyToBuild |= ETargetPlatformReadyStatus::SDKNotFound;
		}
		return bReadyToBuild;
	}

	TARGETPLATFORM_API virtual bool RequiresTempTarget(bool bProjectHasCode, EBuildConfiguration Configuration, bool bRequiresAssetNativization, FText& OutReason) const override;

	virtual bool SupportsValueForType(FName SupportedType, FName RequiredSupportedValue) const override
	{
		// check if the given shader format is returned by this TargetPlatform
		if (SupportedType == TEXT("ShaderFormat"))
		{
			TArray<FName> AllPossibleShaderFormats;
			GetAllPossibleShaderFormats(AllPossibleShaderFormats);
			return AllPossibleShaderFormats.Contains(RequiredSupportedValue);
		}

		return false;
	}

	virtual bool SupportsVariants() const override
	{
		return false;
	}

	virtual float GetVariantPriority() const override
	{
		return IsClientOnly() ? 0.0f : 0.2f;
	}

	virtual bool SendLowerCaseFilePaths() const override
	{
		return false;
	}

	virtual bool AllowsEditorObjects() const override
	{
		return HasEditorOnlyData();
	}

	virtual bool AllowsDevelopmentObjects() const override
	{
		return AllowsEditorObjects();
	}

	virtual void GetBuildProjectSettingKeys(FString& OutSection, TArray<FString>& InBoolKeys, TArray<FString>& InIntKeys, TArray<FString>& InStringKeys) const override
	{
		// do nothing in the base class
	}

	virtual int32 GetPlatformOrdinal() const override
	{
		return PlatformOrdinal;
	}

	TARGETPLATFORM_API virtual TSharedPtr<IDeviceManagerCustomPlatformWidgetCreator> GetCustomWidgetCreator() const override;

	virtual bool ShouldExpandTo32Bit(const uint16* Indices, const int32 NumIndices) const override
	{
		return false;
	}

#if WITH_ENGINE
	virtual bool AllowAudioVisualData() const override
	{
		return !IsServerOnly();
	}

	virtual bool AllowObject(const class UObject* Object) const override
	{
		return true;
	}

	virtual FName GetMeshBuilderModuleName() const override
	{
		// MeshBuilder is the default module. Platforms may override this to provide platform specific mesh data.
		static const FName NAME_MeshBuilder(TEXT("MeshBuilder"));
		return NAME_MeshBuilder;
	}

	virtual void GetShaderFormatModuleHints(TArray<FName>& OutModuleNames) const override
	{
	}
	
	virtual void GetTextureFormatModuleHints(TArray<FName>& OutModuleNames) const override
	{
	}
	
	// beware this is duplicated between TargetPlatformBase and TargetPlatformControlsBase
	TARGETPLATFORM_API virtual void GetTextureSizeLimits(uint64 & OutMaximumSurfaceBytes, uint64 & OutMaximumPackageBytes) const override;

	TARGETPLATFORM_API virtual FName GetWaveFormat(const class USoundWave* Wave) const override;
	
	TARGETPLATFORM_API virtual void GetAllWaveFormats(TArray<FName>& OutFormats) const override;

	TARGETPLATFORM_API virtual void GetWaveFormatModuleHints(TArray<FName>& OutModuleNames) const override;

#endif

	virtual bool CopyFileToTarget(const FString& TargetAddress, const FString& HostFilename, const FString& TargetFilename, const TMap<FString,FString>& CustomPlatformData) override
	{
		return false; 
	}

	virtual void InitializeForCook() override
	{

	}

	virtual void GetExtraPackagesToCook(TArray<FName>& PackageNames) const override
	{

	}

	virtual bool InitializeHostPlatform()
	{
		// if the platform doesn't need anything, it's valid to do nothing
		return true;
	}


protected:

	TARGETPLATFORM_API FTargetPlatformBase(const PlatformInfo::FTargetPlatformInfo *const InPlatformInfo);

	/** Information about this platform */
	const PlatformInfo::FTargetPlatformInfo *PlatformInfo;
	int32 PlatformOrdinal;
	
	TARGETPLATFORM_API const Audio::FAudioFormatSettings& GetAudioFormatSettings() const;

	/** Analytics helper functions */
	TARGETPLATFORM_API static void AppendAnalyticsEventConfigBool( TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride = nullptr );
	TARGETPLATFORM_API static void AppendAnalyticsEventConfigInt( TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride = nullptr );
	TARGETPLATFORM_API static void AppendAnalyticsEventConfigFloat( TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride = nullptr );
	TARGETPLATFORM_API static void AppendAnalyticsEventConfigString( TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride = nullptr );
	TARGETPLATFORM_API static void AppendAnalyticsEventConfigArray( TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride = nullptr );

private:
	TPimplPtr<Audio::FAudioFormatSettings> AudioFormatSettings;
	
	bool HasDefaultBuildSettings() const;
	static bool DoProjectSettingsMatchDefault(const FString& InPlatformName, const FString& InSection, const TArray<FString>* InBoolKeys, const TArray<FString>* InIntKeys, const TArray<FString>* InStringKeys);
};


/**
 * Template for target platforms.
 *
 * @param TPlatformProperties Type of platform properties.
 */
template<typename TPlatformProperties>
class UE_DEPRECATED(5.5, "TTargetPlatformBase is deprecated, please use TTargetPlatformControlsBase(SDK required to build) or TTargetPlatformSettingsBase(no SDK required).") TTargetPlatformBase
	: public FTargetPlatformBase
{
public:

	/**
	 * Returns true if the target platform will be able to be  initialized with an FPlatformInfo. Because FPlatformInfo now comes from a .ini file,
	 * it's possible that the .dll exists, but the .ini does not (should be uncommon, but is necessary to be handled)
	 */
	static bool IsUsable()
	{
		return true;
	}
	


	/**
	 * Constructor that already has a TPI (notably coming from TNonDesktopTargetPlatform)
	 */
	TTargetPlatformBase(PlatformInfo::FTargetPlatformInfo* PremadePlatformInfo)
		: FTargetPlatformBase(PremadePlatformInfo)
	{
		// HasEditorOnlyData and RequiresCookedData are mutually exclusive.
		check(TPlatformProperties::HasEditorOnlyData() != TPlatformProperties::RequiresCookedData());
	}

	/**
	 * Constructor that makes a TPI based solely on TPlatformProperties
	 */
	TTargetPlatformBase()
		: TTargetPlatformBase( new PlatformInfo::FTargetPlatformInfo(
			TPlatformProperties::IniPlatformName(),
			TPlatformProperties::HasEditorOnlyData() ? EBuildTargetType::Editor :
				TPlatformProperties::IsServerOnly() ? EBuildTargetType::Server : 
				TPlatformProperties::IsClientOnly() ? EBuildTargetType::Client : 
				EBuildTargetType::Game,
			TEXT(""))
		)
	{
	}

public:

	// ITargetPlatform interface

	virtual bool HasEditorOnlyData() const override
	{
		return TPlatformProperties::HasEditorOnlyData();
	}

	virtual bool IsLittleEndian() const override
	{
		return TPlatformProperties::IsLittleEndian();
	}

	virtual bool IsServerOnly() const override
	{
		return TPlatformProperties::IsServerOnly();
	}

	virtual bool IsClientOnly() const override
	{
		return TPlatformProperties::IsClientOnly();
	}

	virtual FString PlatformName() const override
	{
		// we assume these match for DesktopPlatforms (NonDesktop doesn't return "FooClient", but Desktop does, for legacy reasons)
		checkSlow(this->PlatformInfo->Name == TPlatformProperties::PlatformName());
		
		return FString(TPlatformProperties::PlatformName());
	}

	virtual FString IniPlatformName() const override
	{
		return FString(TPlatformProperties::IniPlatformName());
	}

	virtual FString CookingDeviceProfileName() const override
	{
		// default to using TargetPlatform name, like WindowsClient, when looking up the DeviceProfile to cook with
		return PlatformName();
	}

	virtual bool RequiresCookedData() const override
	{
		return TPlatformProperties::RequiresCookedData();
	}

	virtual bool RequiresOriginalReleaseVersionForPatch() const override
	{
		return TPlatformProperties::RequiresOriginalReleaseVersionForPatch();
	}

	virtual bool HasNonTrivialPatchGeneration() const override
	{
		return TPlatformProperties::HasNonTrivialPatchGeneration();
	}

	virtual bool HasSecurePackageFormat() const override
	{
		return TPlatformProperties::HasSecurePackageFormat();
	}

	virtual EPlatformAuthentication RequiresUserCredentials() const override
	{
		if (TPlatformProperties::RequiresUserCredentials())
		{
			return EPlatformAuthentication::Always;
		}
		else
		{
			return EPlatformAuthentication::Never;
		}
	}

	virtual bool SupportsBuildTarget( EBuildTargetType TargetType ) const override
	{
		return TPlatformProperties::SupportsBuildTarget(TargetType);
	}

	virtual EBuildTargetType GetRuntimePlatformType() const override
	{
		if (AllowsEditorObjects())
		{
			// Platforms that AllowsEditorObjects need the runtime type Editor to use those objects
			return EBuildTargetType::Editor;
		}
		return PlatformInfo->PlatformType;
	}

	virtual bool SupportsAutoSDK() const override
	{
		return TPlatformProperties::SupportsAutoSDK();
	}

	virtual bool SupportsFeature( ETargetPlatformFeatures Feature ) const override
	{
		switch (Feature)
		{
		case ETargetPlatformFeatures::AudioStreaming:
			return TPlatformProperties::SupportsAudioStreaming();

		case ETargetPlatformFeatures::DistanceFieldShadows:
			return TPlatformProperties::SupportsDistanceFieldShadows();

		case ETargetPlatformFeatures::DistanceFieldAO:
			return TPlatformProperties::SupportsDistanceFieldAO();

		case ETargetPlatformFeatures::GrayscaleSRGB:
			return TPlatformProperties::SupportsGrayscaleSRGB();

		case ETargetPlatformFeatures::HighQualityLightmaps:
			return TPlatformProperties::SupportsHighQualityLightmaps();

		case ETargetPlatformFeatures::LowQualityLightmaps:
			return TPlatformProperties::SupportsLowQualityLightmaps();

		case ETargetPlatformFeatures::MultipleGameInstances:
			return TPlatformProperties::SupportsMultipleGameInstances();

		case ETargetPlatformFeatures::Packaging:
			return false;

		case ETargetPlatformFeatures::CanCookPackages:
			return false;

		case ETargetPlatformFeatures::TextureStreaming:
			return TPlatformProperties::SupportsTextureStreaming();
		case ETargetPlatformFeatures::MeshLODStreaming:
			return TPlatformProperties::SupportsMeshLODStreaming();

		case ETargetPlatformFeatures::MemoryMappedFiles:
			return TPlatformProperties::SupportsMemoryMappedFiles();

		case ETargetPlatformFeatures::MemoryMappedAudio:
			return TPlatformProperties::SupportsMemoryMappedAudio();
		case ETargetPlatformFeatures::MemoryMappedAnimation:
			return TPlatformProperties::SupportsMemoryMappedAnimation();

		case ETargetPlatformFeatures::SdkConnectDisconnect:
		case ETargetPlatformFeatures::UserCredentials:
			break;

		case ETargetPlatformFeatures::MobileRendering:
			return false;
		case ETargetPlatformFeatures::DeferredRendering:
			return true;

		case ETargetPlatformFeatures::ShouldSplitPaksIntoSmallerSizes :
			return false;

		case ETargetPlatformFeatures::HalfFloatVertexFormat:
			return true;

		case ETargetPlatformFeatures::LumenGI:
			return TPlatformProperties::SupportsLumenGI();

		case ETargetPlatformFeatures::HardwareLZDecompression:
			return TPlatformProperties::SupportsHardwareLZDecompression();
		}

		return false;
	}
	virtual FName GetZlibReplacementFormat() const override
	{
		return TPlatformProperties::GetZlibReplacementFormat() != nullptr ? FName(TPlatformProperties::GetZlibReplacementFormat()) : NAME_Zlib;
	}

	virtual int64 GetMemoryMappingAlignment() const override
	{
		return TPlatformProperties::GetMemoryMappingAlignment();
	}

	virtual bool UsesRayTracing() const override
	{
		if (TPlatformProperties::SupportsRayTracing())
		{
			return FTargetPlatformBase::UsesRayTracing();
		}
		return false;
	}


#if WITH_ENGINE
	virtual FName GetPhysicsFormat( class UBodySetup* Body ) const override
	{
		return FName(TPlatformProperties::GetPhysicsFormat());
	}
#endif // WITH_ENGINE
};


template<typename TPlatformProperties>
class UE_DEPRECATED(5.5, "TNonDesktopTargetPlatformBase is deprecated, please use TNonDesktopTargetPlatformControlsBase.") TNonDesktopTargetPlatformBase
	: public TTargetPlatformBase<TPlatformProperties>
{
public:
	/**
	 * A simplified version for TPs that never will have Editor or ServerOnly versions, potentially multiple CookFlavors, as well as IN VERY RARE CASES, 
	 * a different runtime IniPlatformName than what is passed in here (an example being TVOS and IOS, where passing in TVOS properties is very complicated)
	 * Note that if we delayed the Info creation, we could just use this->IniPlatformName() and override that in, say TVOS, but we can't call a virtual here,
	 * so we pass it up into the ctor
	 */
	TNonDesktopTargetPlatformBase(bool bInIsClientOnly, const TCHAR* CookFlavor=nullptr, const TCHAR* OverrideIniPlatformName=nullptr)
		: TTargetPlatformBase<TPlatformProperties>(new PlatformInfo::FTargetPlatformInfo(
			OverrideIniPlatformName ? FString(OverrideIniPlatformName) : FString(TPlatformProperties::IniPlatformName()),
			bInIsClientOnly ? EBuildTargetType::Client : EBuildTargetType::Game,
			CookFlavor))
		, bIsClientOnly(bInIsClientOnly)
	{

	}

	virtual FString PlatformName() const override
	{
		// instead of TPlatformProperties (which won't have Client for non-desktop platforms), use the Info's name, which is programmatically made
		return this->PlatformInfo->Name.ToString();
	}

	virtual FString IniPlatformName() const override
	{
		// we use the Info's IniPlatformName as it may have been overridden in the constructor IN RARE CASES
		return this->PlatformInfo->IniPlatformName.ToString();
	}

	virtual FString CookingDeviceProfileName() const override
	{
		// when cooking for non-desktop platforms, always use the base platform name as the DP to cook with (there aren't IOSClient DPs, there are IOS (base), IPhone7, iPad4, etc DPs)
		return IniPlatformName();
	}

	virtual bool HasEditorOnlyData() const override
	{
		return false;
	}

	virtual bool IsServerOnly() const override
	{
		return false;
	}

	virtual bool IsClientOnly() const override
	{
		return bIsClientOnly;
	}

	virtual bool IsRunningPlatform() const override
	{
		// IsRunningPlatform is only for editor platforms
		return false;
	}

protected:
	// true if this target platform is client-only, ie strips out server stuff
	bool bIsClientOnly;
};


// Redirect everything to TPS or TPC
class FTargetPlatformMerged final
	: public ITargetPlatform
{
public:
	FTargetPlatformMerged(ITargetPlatformSettings* InTargetPlatformSettings, ITargetPlatformControls* InTargetPlatformControls)
		: TargetPlatformSettings(InTargetPlatformSettings)
		, TargetPlatformControls(InTargetPlatformControls)
	{

	}
	virtual bool AddDevice(const FString& DeviceName, bool bDefault) override 
	{ 
		return TargetPlatformControls->AddDevice(DeviceName, bDefault); 
	}
	virtual bool AddDevice(const FString& DeviceId, const FString& DeviceUserFriendlyName, const FString& Username, const FString& Password, bool bDefault) override 
	{ 
		return TargetPlatformControls->AddDevice(DeviceId, DeviceUserFriendlyName, Username, Password, bDefault); 
	}
	virtual FString PlatformName() const override 
	{
		return TargetPlatformControls->PlatformName();
	}
	virtual FText DisplayName() const override 
	{
		return TargetPlatformControls->DisplayName();
	}
	virtual int32 CheckRequirements(bool bProjectHasCode, EBuildConfiguration Configuration, bool bRequiresAssetNativization, FString& OutTutorialPath, FString& OutDocumentationPath, FText& CustomizedLogMessage) const override 
	{
		return TargetPlatformControls->CheckRequirements(bProjectHasCode, Configuration, bRequiresAssetNativization, OutTutorialPath, OutDocumentationPath, CustomizedLogMessage);
	}
	virtual bool RequiresTempTarget(bool bProjectHasCode, EBuildConfiguration Configuration, bool bRequiresAssetNativization, FText& OutReason) const override 
	{
		return TargetPlatformControls->RequiresTempTarget(bProjectHasCode, Configuration, bRequiresAssetNativization, OutReason);
	}
	virtual const PlatformInfo::FTargetPlatformInfo& GetTargetPlatformInfo() const override 
	{
		return TargetPlatformControls->GetTargetPlatformInfo();
	}
	virtual const FDataDrivenPlatformInfo& GetPlatformInfo() const override 
	{
		return TargetPlatformControls->GetPlatformInfo();
	}
	virtual FConfigCacheIni* GetConfigSystem() const override 
	{
		return TargetPlatformSettings->GetConfigSystem();
	}
	virtual FString IniPlatformName() const override 
	{
		return TargetPlatformSettings->IniPlatformName();
	}
	virtual FString CookingDeviceProfileName() const override 
	{
		return TargetPlatformControls->CookingDeviceProfileName();
	}
	virtual void EnableDeviceCheck(bool OnOff) override 
	{
		TargetPlatformControls->EnableDeviceCheck(OnOff);
	}
	virtual void GetAllDevices(TArray<ITargetDevicePtr>& OutDevices) const override 
	{
		TargetPlatformControls->GetAllDevices(OutDevices);
	}
	virtual void RefreshAllDevices() override
	{
		TargetPlatformControls->RefreshAllDevices();
	}
	virtual FName GetZlibReplacementFormat() const override 
	{
		return TargetPlatformControls->GetZlibReplacementFormat();
	}
	virtual int64 GetMemoryMappingAlignment() const override 
	{
		return TargetPlatformControls->GetMemoryMappingAlignment();
	}
	virtual bool GenerateStreamingInstallManifest(const TMultiMap<FString, int32>& PakchunkMap, const TSet<int32>& PakchunkIndicesInUse) const override 
	{
		return TargetPlatformControls->GenerateStreamingInstallManifest(PakchunkMap, PakchunkIndicesInUse);
	}
	virtual ITargetDevicePtr GetDefaultDevice() const override 
	{
		return TargetPlatformControls->GetDefaultDevice();
	}
	virtual ITargetDevicePtr GetDevice(const FTargetDeviceId& DeviceId) override 
	{
		return TargetPlatformControls->GetDevice(DeviceId);
	}
	virtual bool HasEditorOnlyData() const override 
	{
		return TargetPlatformControls->HasEditorOnlyData();
	}
	virtual bool AllowsEditorObjects() const override 
	{
		return TargetPlatformControls->AllowsEditorObjects();
	}
	virtual bool AllowsDevelopmentObjects() const override 
	{
		return TargetPlatformControls->AllowsDevelopmentObjects();
	}
	virtual bool IsClientOnly() const override 
	{
		return TargetPlatformControls->IsClientOnly();
	}
	virtual bool IsLittleEndian() const override 
	{
		return TargetPlatformControls->IsLittleEndian();
	}
	virtual bool IsRunningPlatform() const override 
	{
		return TargetPlatformSettings->IsRunningPlatform();
	}
	virtual bool IsServerOnly() const override 
	{
		return TargetPlatformControls->IsServerOnly();
	}
	virtual bool IsEnabledForPlugin(const IPlugin& Plugin) const override 
	{
		return TargetPlatformControls->IsEnabledForPlugin(Plugin);
	}
	virtual bool CanSupportRemoteShaderCompile() const override 
	{
		return TargetPlatformControls->CanSupportRemoteShaderCompile();
	}
	virtual void GetShaderCompilerDependencies(TArray<FString>& OutDependencies) const override 
	{
		TargetPlatformControls->GetShaderCompilerDependencies(OutDependencies);
	}
	virtual bool IsSdkInstalled(bool bProjectHasCode, FString& OutDocumentationPath) const override 
	{
		return TargetPlatformControls->IsSdkInstalled(bProjectHasCode, OutDocumentationPath);
	}
	virtual bool RequiresCookedData() const override 
	{
		return TargetPlatformControls->RequiresCookedData();
	}
	virtual bool RequiresOriginalReleaseVersionForPatch() const override 
	{
		return TargetPlatformControls->RequiresOriginalReleaseVersionForPatch();
	}
	virtual bool HasNonTrivialPatchGeneration() const override 
	{
		return TargetPlatformControls->HasNonTrivialPatchGeneration();
	}
	virtual bool HasSecurePackageFormat() const override 
	{
		return TargetPlatformControls->HasSecurePackageFormat();
	}
	virtual EPlatformAuthentication RequiresUserCredentials() const override 
	{
		return TargetPlatformControls->RequiresUserCredentials();
	}
	virtual bool SupportsAutoSDK() const override 
	{
		return TargetPlatformControls->SupportsAutoSDK();
	}
	virtual bool SupportsBuildTarget(EBuildTargetType TargetType) const override 
	{
		return TargetPlatformControls->SupportsBuildTarget(TargetType);
	}
	virtual EBuildTargetType GetRuntimePlatformType() const override 
	{
		return TargetPlatformControls->GetRuntimePlatformType();
	}
	virtual bool SupportsFeature(ETargetPlatformFeatures Feature) const override 
	{
		return TargetPlatformSettings->SupportsFeature(Feature);
	}
	virtual bool SupportsValueForType(FName SupportedType, FName RequiredSupportedValue) const override 
	{
		return TargetPlatformSettings->SupportsValueForType(SupportedType, RequiredSupportedValue);
	}
	virtual bool UsesForwardShading() const override 
	{
		return TargetPlatformSettings->UsesForwardShading();
	}
	virtual bool UsesDBuffer() const override 
	{
		return TargetPlatformSettings->UsesDBuffer();
	}
	virtual bool UsesBasePassVelocity() const override 
	{
		return TargetPlatformSettings->UsesBasePassVelocity();
	}
	virtual bool UsesSelectiveBasePassOutputs() const override 
	{
		return TargetPlatformSettings->UsesSelectiveBasePassOutputs();
	}
	virtual bool ShouldStripNaniteFallbackMeshes() const override
	{
		return TargetPlatformSettings->ShouldStripNaniteFallbackMeshes();
	}
	virtual bool UsesDistanceFields() const override 
	{
		return TargetPlatformSettings->UsesDistanceFields();
	}
	virtual bool UsesRayTracing() const override 
	{
		return TargetPlatformSettings->UsesRayTracing();
	}
	virtual ERayTracingRuntimeMode GetRayTracingMode() const override
	{
		return TargetPlatformSettings->GetRayTracingMode();
	}
	virtual uint32 GetSupportedHardwareMask() const override
	{
		return TargetPlatformSettings->GetSupportedHardwareMask();
	}
	virtual EOfflineBVHMode GetStaticMeshOfflineBVHMode() const override 
	{
		return TargetPlatformSettings->GetStaticMeshOfflineBVHMode();
	}
	virtual bool GetStaticMeshOfflineBVHCompression() const override 
	{
		return TargetPlatformSettings->GetStaticMeshOfflineBVHCompression();
	}
	virtual EOfflineBVHMode GetSkeletalMeshOfflineBVHMode() const override
	{
		return TargetPlatformSettings->GetSkeletalMeshOfflineBVHMode();
	}
	virtual bool ForcesSimpleSkyDiffuse() const override 
	{
		return TargetPlatformSettings->ForcesSimpleSkyDiffuse();
	}
	virtual bool VelocityEncodeDepth() const override 
	{
		return TargetPlatformSettings->VelocityEncodeDepth();
	}
	virtual float GetDownSampleMeshDistanceFieldDivider() const override 
	{
		return TargetPlatformSettings->GetDownSampleMeshDistanceFieldDivider();
	}
	virtual int32 GetHeightFogModeForOpaque() const override 
	{
		return TargetPlatformSettings->GetHeightFogModeForOpaque();
	}
	virtual bool UsesMobileDBuffer() const override 
	{
		return TargetPlatformSettings->UsesMobileDBuffer();
	}
	virtual bool UsesASTCHDR() const override 
	{
		return TargetPlatformSettings->UsesASTCHDR();
	}
	virtual FName GetFallbackASTCHDR() const override
	{
		return TargetPlatformSettings->GetFallbackASTCHDR();
	}
	virtual void GetAllPossibleShaderFormats(TArray<FName>& OutFormats) const override 
	{
		TargetPlatformSettings->GetAllPossibleShaderFormats(OutFormats);
	}
	virtual void GetAllTargetedShaderFormats(TArray<FName>& OutFormats) const override 
	{
		TargetPlatformSettings->GetAllTargetedShaderFormats(OutFormats);
	}
	virtual void GetRayTracingShaderFormats(TArray<FName>& OutFormats) const override 
	{
		TargetPlatformSettings->GetRayTracingShaderFormats(OutFormats);
	}
	virtual void GetPossibleArchitectures(TArray<FString>& OutArchitectures) const override
	{
		TargetPlatformSettings->GetPossibleArchitectures(OutArchitectures);
	}
	virtual FString GetHostArchitecture() const override
	{
		return TargetPlatformSettings->GetHostArchitecture();
	}
	virtual void GetPlatformSpecificProjectAnalytics(TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray) const override 
	{
		TargetPlatformControls->GetPlatformSpecificProjectAnalytics(AnalyticsParamArray);
	}
#if WITH_ENGINE
	virtual FName GetPhysicsFormat(class UBodySetup* Body) const override 
	{
		return TargetPlatformControls->GetPhysicsFormat(Body);
	}
	virtual void GetReflectionCaptureFormats(TArray<FName>& OutFormats) const override 
	{
		TargetPlatformSettings->GetReflectionCaptureFormats(OutFormats);
	}
	virtual void GetShaderFormatModuleHints(TArray<FName>& OutModuleNames) const override 
	{
		TargetPlatformSettings->GetShaderFormatModuleHints(OutModuleNames);
	}
	virtual void GetTextureFormats(const class UTexture* Texture, TArray< TArray<FName> >& OutFormats) const override 
	{
		TargetPlatformControls->GetTextureFormats(Texture, OutFormats);
	}
	
	virtual void GetTextureSizeLimits(uint64 & OutMaximumSurfaceBytes, uint64 & OutMaximumPackageBytes) const override
	{
		TargetPlatformControls->GetTextureSizeLimits(OutMaximumSurfaceBytes,OutMaximumPackageBytes);
	}

	virtual void GetTextureFormatModuleHints(TArray<FName>& OutModuleNames) const override 
	{
		TargetPlatformControls->GetTextureFormatModuleHints(OutModuleNames);
	}
	virtual FName FinalizeVirtualTextureLayerFormat(FName Format) const override 
	{
		return TargetPlatformControls->FinalizeVirtualTextureLayerFormat(Format);
	}
	virtual bool SupportsLQCompressionTextureFormat() const override 
	{
		return TargetPlatformControls->SupportsLQCompressionTextureFormat();
	}
	virtual FName GetWaveFormat(const class USoundWave* Wave) const override 
	{
		return TargetPlatformControls->GetWaveFormat(Wave);
	}
	virtual void GetAllWaveFormats(TArray<FName>& OutFormats) const override 
	{
		TargetPlatformControls->GetAllWaveFormats(OutFormats);
	}
	virtual void GetWaveFormatModuleHints(TArray<FName>& OutModuleNames) const override 
	{
		TargetPlatformControls->GetWaveFormatModuleHints(OutModuleNames);
	}
	virtual bool AllowAudioVisualData() const override 
	{
		return TargetPlatformControls->AllowAudioVisualData();
	}
	virtual bool AllowObject(const class UObject* Object) const override 
	{
		return TargetPlatformControls->AllowObject(Object);
	}
	virtual const class UTextureLODSettings& GetTextureLODSettings() const override 
	{
		return TargetPlatformSettings->GetTextureLODSettings();
	}
	virtual void RegisterTextureLODSettings(const class UTextureLODSettings* InTextureLODSettings) override 
	{
		TargetPlatformSettings->RegisterTextureLODSettings(InTextureLODSettings);
	}
	virtual const class FStaticMeshLODSettings& GetStaticMeshLODSettings() const override 
	{
		return TargetPlatformSettings->GetStaticMeshLODSettings();
	}
	virtual FName GetMeshBuilderModuleName() const override 
	{
		return TargetPlatformControls->GetMeshBuilderModuleName();
	}
#endif
	virtual bool PackageBuild(const FString& InPackgeDirectory) override 
	{
		return TargetPlatformControls->PackageBuild(InPackgeDirectory);
	}
	virtual bool SupportsVariants() const override 
	{
		return TargetPlatformControls->SupportsVariants();
	}
	virtual float GetVariantPriority() const override 
	{
		return TargetPlatformControls->SupportsVariants();
	}
	virtual bool SendLowerCaseFilePaths() const override 
	{
		return TargetPlatformControls->SupportsVariants();
	}
	virtual void GetBuildProjectSettingKeys(FString& OutSection, TArray<FString>& InBoolKeys, TArray<FString>& InIntKeys, TArray<FString>& InStringKeys) const override 
	{
		TargetPlatformControls->GetBuildProjectSettingKeys(OutSection, InBoolKeys, InIntKeys, InStringKeys);
	}
	virtual int32 GetPlatformOrdinal() const override 
	{
		return TargetPlatformControls->GetPlatformOrdinal();
	}
	virtual TSharedPtr<IDeviceManagerCustomPlatformWidgetCreator> GetCustomWidgetCreator() const override 
	{
		return TargetPlatformControls->GetCustomWidgetCreator();
	}
	virtual bool ShouldExpandTo32Bit(const uint16* Indices, const int32 NumIndices) const override 
	{
		return TargetPlatformControls->ShouldExpandTo32Bit(Indices, NumIndices);
	}
	virtual bool CopyFileToTarget(const FString& DeviceId, const FString& HostFilename, const FString& TargetFilename, const TMap<FString, FString>& CustomPlatformData) override 
	{
		return TargetPlatformControls->CopyFileToTarget(DeviceId, HostFilename, TargetFilename, CustomPlatformData);
	}
	virtual void InitializeForCook() override
	{
		TargetPlatformControls->InitializeForCook();
	}
	virtual void GetExtraPackagesToCook(TArray<FName>& PackageNames) const override
	{
		TargetPlatformControls->GetExtraPackagesToCook(PackageNames);
	}
	virtual bool InitializeHostPlatform() override 
	{
		return TargetPlatformControls->InitializeHostPlatform();
	}

public:

	/** Virtual destructor. */
	virtual ~FTargetPlatformMerged() = default;

private:
	ITargetPlatformSettings* TargetPlatformSettings;
	ITargetPlatformControls* TargetPlatformControls;
};
