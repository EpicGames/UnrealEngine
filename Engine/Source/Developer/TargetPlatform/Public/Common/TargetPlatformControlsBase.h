// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/ITargetPlatformControls.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "CoreGlobals.h"
#include "PlatformInfo.h"
#include "Interfaces/ITargetDevice.h"
#include "Interfaces/ITargetPlatformSettings.h"
#include "Templates/PimplPtr.h"
#include "Misc/Paths.h"
#include "Misc/ConfigCacheIni.h"
// Forward declare.
namespace Audio
{
	class FAudioFormatSettings;
}

class FTargetPlatformControlsBase
	: public ITargetPlatformControls
{
public:
	virtual bool AddDevice(const FString& DeviceName, bool bDefault) override
	{
		return false;
	}

	virtual bool AddDevice(const FString& DeviceId, const FString& DeviceUserFriendlyName, const FString& Username, const FString& Password, bool bDefault) override
	{
		return AddDevice(DeviceId, bDefault);
	}

	virtual FText DisplayName() const override
	{
		return PlatformInfo->DisplayName;
	}

	virtual const PlatformInfo::FTargetPlatformInfo& GetTargetPlatformInfo() const override
	{
		return *PlatformInfo;
	}

	virtual const FDataDrivenPlatformInfo& GetPlatformInfo() const override
	{
		return *PlatformInfo->DataDrivenPlatformInfo;
	}

	TARGETPLATFORM_API virtual bool IsEnabledForPlugin(const IPlugin& Plugin) const override;
	TARGETPLATFORM_API virtual void GetPlatformSpecificProjectAnalytics(TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray) const override;

#if WITH_ENGINE
	virtual FName FinalizeVirtualTextureLayerFormat(FName Format) const override
	{
		return Format;
	}
	virtual bool SupportsLQCompressionTextureFormat() const override { return true; };
#endif

	virtual bool PackageBuild(const FString& InPackgeDirectory) override
	{
		return true;
	}

	virtual bool CanSupportRemoteShaderCompile() const override
	{
		return true;
	}

	virtual void GetShaderCompilerDependencies(TArray<FString>& OutDependencies) const override
	{
	}

	/** Helper method to fill a dependencies array for the shader compiler with absolute paths, passing a relative path to the Engine as the parameter. */
	static void AddDependencySCArrayHelper(TArray<FString>& OutDependencies, const FString& DependencyRelativePath)
	{
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		FString DependencyAbsolutePath = PlatformFile.ConvertToAbsolutePathForExternalAppForRead(*(FPaths::EngineDir() / DependencyRelativePath));
		FPaths::NormalizeDirectoryName(DependencyAbsolutePath);
		OutDependencies.AddUnique(DependencyAbsolutePath);
	}

	virtual bool IsSdkInstalled(bool bProjectHasCode, FString& OutDocumentationPath) const override
	{
		return true;
	}

	virtual int32 CheckRequirements(bool bProjectHasCode, EBuildConfiguration Configuration, bool bRequiresAssetNativization, FString& OutTutorialPath, FString& OutDocumentationPath, FText& CustomizedLogMessage) const override
	{
		int32 bReadyToBuild = ETargetPlatformReadyStatus::Ready; // @todo How do we check that the iOS SDK is installed when building from Windows? Is that even possible?
		if (!IsSdkInstalled(bProjectHasCode, OutTutorialPath))
		{
			bReadyToBuild |= ETargetPlatformReadyStatus::SDKNotFound;
		}
		return bReadyToBuild;
	}

	TARGETPLATFORM_API virtual bool RequiresTempTarget(bool bProjectHasCode, EBuildConfiguration Configuration, bool bRequiresAssetNativization, FText& OutReason) const override;

	virtual bool SupportsVariants() const override
	{
		return false;
	}

	virtual float GetVariantPriority() const override
	{
		return IsClientOnly() ? 0.0f : 0.2f;
	}

	virtual bool SendLowerCaseFilePaths() const override
	{
		return false;
	}

	virtual bool AllowsEditorObjects() const override
	{
		return HasEditorOnlyData();
	}

	virtual bool AllowsDevelopmentObjects() const override
	{
		return AllowsEditorObjects();
	}

	virtual void GetBuildProjectSettingKeys(FString& OutSection, TArray<FString>& InBoolKeys, TArray<FString>& InIntKeys, TArray<FString>& InStringKeys) const override
	{
		// do nothing in the base class
	}

	virtual int32 GetPlatformOrdinal() const override
	{
		return PlatformOrdinal;
	}

	TARGETPLATFORM_API virtual TSharedPtr<IDeviceManagerCustomPlatformWidgetCreator> GetCustomWidgetCreator() const override;

	virtual bool ShouldExpandTo32Bit(const uint16* Indices, const int32 NumIndices) const override
	{
		return false;
	}

#if WITH_ENGINE
	virtual bool AllowAudioVisualData() const override
	{
		return !IsServerOnly();
	}

	virtual bool AllowObject(const class UObject* Object) const override
	{
		return true;
	}

	virtual FName GetMeshBuilderModuleName() const override
	{
		// MeshBuilder is the default module. Platforms may override this to provide platform specific mesh data.
		static const FName NAME_MeshBuilder(TEXT("MeshBuilder"));
		return NAME_MeshBuilder;
	}
	
	// beware this is duplicated between TargetPlatformBase and TargetPlatformControlsBase
	virtual void GetTextureFormatModuleHints(TArray<FName>& OutModuleNames) const override
	{
	}

	TARGETPLATFORM_API virtual FName GetWaveFormat(const class USoundWave* Wave) const override;

	TARGETPLATFORM_API virtual void GetAllWaveFormats(TArray<FName>& OutFormats) const override;

	TARGETPLATFORM_API virtual void GetWaveFormatModuleHints(TArray<FName>& OutModuleNames) const override;
	
	static TARGETPLATFORM_API void GetTextureSizeLimitsDefault(FConfigCacheIni* ConfigSystem,uint64 & OutMaximumSurfaceBytes, uint64 & OutMaximumPackageBytes);
	
	// beware this is duplicated between TargetPlatformBase and TargetPlatformControlsBase
	virtual void GetTextureSizeLimits(uint64 & OutMaximumSurfaceBytes, uint64 & OutMaximumPackageBytes) const override
	{
		GetTextureSizeLimitsDefault(TargetPlatformSettings->GetConfigSystem(),OutMaximumSurfaceBytes,OutMaximumPackageBytes);
	}

#endif

	virtual bool CopyFileToTarget(const FString& TargetAddress, const FString& HostFilename, const FString& TargetFilename, const TMap<FString, FString>& CustomPlatformData) override
	{
		return false;
	}

	virtual void InitializeForCook() override
	{

	}

	virtual void GetExtraPackagesToCook(TArray<FName>& PackageNames) const override
	{

	}

	virtual bool InitializeHostPlatform()
	{
		// if the platform doesn't need anything, it's valid to do nothing
		return true;
	}

protected:
	TARGETPLATFORM_API FTargetPlatformControlsBase(const PlatformInfo::FTargetPlatformInfo* const InPlatformInfo, ITargetPlatformSettings* TargetPlatformSettings);

	/** Information about this platform */
	const PlatformInfo::FTargetPlatformInfo* PlatformInfo;
	int32 PlatformOrdinal;

	TARGETPLATFORM_API const Audio::FAudioFormatSettings& GetAudioFormatSettings() const;

	/** Analytics helper functions */
	TARGETPLATFORM_API static void AppendAnalyticsEventConfigBool(TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride = nullptr);
	TARGETPLATFORM_API static void AppendAnalyticsEventConfigInt(TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride = nullptr);
	TARGETPLATFORM_API static void AppendAnalyticsEventConfigFloat(TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride = nullptr);
	TARGETPLATFORM_API static void AppendAnalyticsEventConfigString(TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride = nullptr);
	TARGETPLATFORM_API static void AppendAnalyticsEventConfigArray(TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray, const TCHAR* ConfigSection, const TCHAR* ConfigKey, const FString& IniFileName, const TCHAR* AnalyticsKeyNameOverride = nullptr);

private:
	TPimplPtr<Audio::FAudioFormatSettings> AudioFormatSettings;

	bool HasDefaultBuildSettings() const;
	static bool DoProjectSettingsMatchDefault(const FString& InPlatformName, const FString& InSection, const TArray<FString>* InBoolKeys, const TArray<FString>* InIntKeys, const TArray<FString>* InStringKeys);
	static bool IsPluginEnabledForTarget(const IPlugin& Plugin, const struct FProjectDescriptor* Project, const FString& Platform, EBuildConfiguration Configuration, EBuildTargetType TargetType);
};

template<typename TPlatformProperties>
class TTargetPlatformControlsBase
	: public FTargetPlatformControlsBase
{
public:
	/**
	 * Returns true if the target platform will be able to be  initialized with an FPlatformInfo. Because FPlatformInfo now comes from a .ini file,
	 * it's possible that the .dll exists, but the .ini does not (should be uncommon, but is necessary to be handled)
	 */
	static bool IsUsable()
	{
		return true;
	}

	/**
	 * Constructor that already has a TPI (notably coming from TNonDesktopTargetPlatform)
	 */
	TTargetPlatformControlsBase(const PlatformInfo::FTargetPlatformInfo* const InPlatformInfo, ITargetPlatformSettings* TargetPlatformSettings)
		: FTargetPlatformControlsBase(InPlatformInfo, TargetPlatformSettings)
	{
		// HasEditorOnlyData and RequiresCookedData are mutually exclusive.
		check(TPlatformProperties::HasEditorOnlyData() != TPlatformProperties::RequiresCookedData());
	}

	/**
	 * Constructor that makes a TPCI based solely on TPlatformProperties
	 */
	TTargetPlatformControlsBase(ITargetPlatformSettings* TargetPlatformSettings)
		: TTargetPlatformControlsBase(new PlatformInfo::FTargetPlatformInfo(
			TPlatformProperties::IniPlatformName(),
			TPlatformProperties::HasEditorOnlyData() ? EBuildTargetType::Editor :
			TPlatformProperties::IsServerOnly() ? EBuildTargetType::Server :
			TPlatformProperties::IsClientOnly() ? EBuildTargetType::Client :
			EBuildTargetType::Game,
			TEXT("")),
			TargetPlatformSettings
		)
	{
	}

	virtual bool HasEditorOnlyData() const override
	{
		return TPlatformProperties::HasEditorOnlyData();
	}

	virtual bool IsLittleEndian() const override
	{
		return TPlatformProperties::IsLittleEndian();
	}

	virtual bool IsServerOnly() const override
	{
		return TPlatformProperties::IsServerOnly();
	}

	virtual bool IsClientOnly() const override
	{
		return TPlatformProperties::IsClientOnly();
	}

	virtual FString PlatformName() const override
	{
		// we assume these match for DesktopPlatforms (NonDesktop doesn't return "FooClient", but Desktop does, for legacy reasons)
		checkSlow(this->PlatformInfo->Name == TPlatformProperties::PlatformName());

		return FString(TPlatformProperties::PlatformName());
	}

	virtual FString CookingDeviceProfileName() const override
	{
		// default to using TargetPlatform name, like WindowsClient, when looking up the DeviceProfile to cook with
		return PlatformName();
	}

	virtual bool RequiresCookedData() const override
	{
		return TPlatformProperties::RequiresCookedData();
	}

	virtual bool RequiresOriginalReleaseVersionForPatch() const override
	{
		return TPlatformProperties::RequiresOriginalReleaseVersionForPatch();
	}

	virtual bool HasNonTrivialPatchGeneration() const override
	{
		return TPlatformProperties::HasNonTrivialPatchGeneration();
	}

	virtual bool HasSecurePackageFormat() const override
	{
		return TPlatformProperties::HasSecurePackageFormat();
	}

	virtual EPlatformAuthentication RequiresUserCredentials() const override
	{
		if (TPlatformProperties::RequiresUserCredentials())
		{
			return EPlatformAuthentication::Always;
		}
		else
		{
			return EPlatformAuthentication::Never;
		}
	}

	virtual bool SupportsBuildTarget(EBuildTargetType TargetType) const override
	{
		return TPlatformProperties::SupportsBuildTarget(TargetType);
	}

	virtual EBuildTargetType GetRuntimePlatformType() const override
	{
		if (AllowsEditorObjects())
		{
			// Platforms that AllowsEditorObjects need the runtime type Editor to use those objects
			return EBuildTargetType::Editor;
		}
		return PlatformInfo->PlatformType;
	}

	virtual bool SupportsAutoSDK() const override
	{
		return TPlatformProperties::SupportsAutoSDK();
	}

	virtual FName GetZlibReplacementFormat() const override
	{
		return TPlatformProperties::GetZlibReplacementFormat() != nullptr ? FName(TPlatformProperties::GetZlibReplacementFormat()) : NAME_Zlib;
	}

	virtual int64 GetMemoryMappingAlignment() const override
	{
		return TPlatformProperties::GetMemoryMappingAlignment();
	}

#if WITH_ENGINE
	virtual FName GetPhysicsFormat(class UBodySetup* Body) const override
	{
		return FName(TPlatformProperties::GetPhysicsFormat());
	}
#endif // WITH_ENGINE
};

template<typename TPlatformProperties>
class TNonDesktopTargetPlatformControlsBase
	: public TTargetPlatformControlsBase<TPlatformProperties>
{
public:
	TNonDesktopTargetPlatformControlsBase(bool bInIsClientOnly, ITargetPlatformSettings* InTargetPlatformSettings, const TCHAR* CookFlavor = nullptr, const TCHAR* OverrideIniPlatformName = nullptr)
		: TTargetPlatformControlsBase<TPlatformProperties>(new PlatformInfo::FTargetPlatformInfo(
			OverrideIniPlatformName ? FString(OverrideIniPlatformName) : FString(TPlatformProperties::IniPlatformName()),
			bInIsClientOnly ? EBuildTargetType::Client : EBuildTargetType::Game,
			CookFlavor), InTargetPlatformSettings)
		,	bIsClientOnly(bInIsClientOnly)
	{

	}
	virtual FString PlatformName() const override
	{
		// instead of TPlatformProperties (which won't have Client for non-desktop platforms), use the Info's name, which is programmatically made
		return this->PlatformInfo->Name.ToString();
	}

	virtual FString CookingDeviceProfileName() const override
	{
		// when cooking for non-desktop platforms, always use the base platform name as the DP to cook with (there aren't IOSClient DPs, there are IOS (base), IPhone7, iPad4, etc DPs)
		return  this->PlatformInfo->IniPlatformName.ToString();
	}

	virtual bool HasEditorOnlyData() const override
	{
		return false;
	}

	virtual bool IsServerOnly() const override
	{
		return false;
	}

	virtual bool IsClientOnly() const override
	{
		return bIsClientOnly;
	}

protected:
	// true if this target platform is client-only, ie strips out server stuff
	bool bIsClientOnly;
};
