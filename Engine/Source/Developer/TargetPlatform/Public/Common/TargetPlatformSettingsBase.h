// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/ITargetPlatformSettings.h"
#include "Misc/ConfigCacheIni.h"
// Forward declare.
namespace Audio
{
	class FAudioFormatSettings;
}

class FTargetPlatformSettingsBase
	: public ITargetPlatformSettings
{
public:
	virtual FConfigCacheIni* GetConfigSystem() const override
	{
		return FConfigCacheIni::ForPlatform(*IniPlatformName());
	}

	TARGETPLATFORM_API virtual bool UsesForwardShading() const override;
	TARGETPLATFORM_API virtual bool UsesDBuffer() const override;
	UE_DEPRECATED(5.1, "Use IsUsingBasePassVelocity(EShaderPlatform Platform) in renderutils that will uses FShaderPlatformCachedIniValue to retrieve the cvar value per platform.")
	TARGETPLATFORM_API virtual bool UsesBasePassVelocity() const override;
	TARGETPLATFORM_API virtual bool VelocityEncodeDepth() const override;
	TARGETPLATFORM_API virtual bool UsesSelectiveBasePassOutputs() const override;
	TARGETPLATFORM_API virtual bool ShouldStripNaniteFallbackMeshes() const override;
	TARGETPLATFORM_API virtual bool UsesDistanceFields() const override;
	TARGETPLATFORM_API virtual bool UsesRayTracing() const override;
	TARGETPLATFORM_API virtual ERayTracingRuntimeMode GetRayTracingMode() const override;
	TARGETPLATFORM_API virtual uint32 GetSupportedHardwareMask() const override;
	TARGETPLATFORM_API virtual EOfflineBVHMode GetStaticMeshOfflineBVHMode() const override;
	TARGETPLATFORM_API virtual bool GetStaticMeshOfflineBVHCompression() const override;
	TARGETPLATFORM_API virtual EOfflineBVHMode GetSkeletalMeshOfflineBVHMode() const override;
	TARGETPLATFORM_API virtual bool ForcesSimpleSkyDiffuse() const override;
	TARGETPLATFORM_API virtual float GetDownSampleMeshDistanceFieldDivider() const override;
	TARGETPLATFORM_API virtual int32 GetHeightFogModeForOpaque() const override;
	TARGETPLATFORM_API virtual bool UsesMobileDBuffer() const override;
	TARGETPLATFORM_API virtual bool UsesASTCHDR() const override;
	TARGETPLATFORM_API virtual FName GetFallbackASTCHDR() const override;
	TARGETPLATFORM_API virtual void GetRayTracingShaderFormats(TArray<FName>& OutFormats) const override;
	TARGETPLATFORM_API virtual void GetPossibleArchitectures(TArray<FString>& OutArchitectures) const override;
	TARGETPLATFORM_API virtual FString GetHostArchitecture() const override;

#if WITH_ENGINE
	virtual void GetShaderFormatModuleHints(TArray<FName>& OutModuleNames) const override
	{
	}

	virtual void GetReflectionCaptureFormats(TArray<FName>& OutFormats) const override
	{
		OutFormats.Add(FName(TEXT("FullHDR")));
	}
#endif

	virtual bool SupportsValueForType(FName SupportedType, FName RequiredSupportedValue) const override
	{
		// check if the given shader format is returned by this TargetPlatform
		if (SupportedType == TEXT("ShaderFormat"))
		{
			TArray<FName> AllPossibleShaderFormats;
			GetAllPossibleShaderFormats(AllPossibleShaderFormats);
			return AllPossibleShaderFormats.Contains(RequiredSupportedValue);
		}

		return false;
	}

	virtual bool IsRunningPlatform() const override
	{
		// IsRunningPlatform is only for editor platforms
		return false;
	}

protected:

	TARGETPLATFORM_API ERayTracingRuntimeMode ParseRuntimeRayTracingMode(const TCHAR* SectionName) const;

	FTargetPlatformSettingsBase() {};
};

template<typename TPlatformProperties>
class TTargetPlatformSettingsBase
	: public FTargetPlatformSettingsBase
{
public:
	TTargetPlatformSettingsBase(const TCHAR* CookFlavor = nullptr, const TCHAR* OverrideIniPlatformName = nullptr)
		: FTargetPlatformSettingsBase()
	{
		IniPlatformNameValue = OverrideIniPlatformName != nullptr ? FString(OverrideIniPlatformName): FString(TPlatformProperties::IniPlatformName());
	}

	virtual FString IniPlatformName() const override
	{
		return IniPlatformNameValue;
	}

	virtual bool UsesRayTracing() const override
	{
		if (TPlatformProperties::SupportsRayTracing())
		{
			return FTargetPlatformSettingsBase::UsesRayTracing();
		}
		return false;
	}

	virtual bool SupportsFeature(ETargetPlatformFeatures Feature) const override
	{
		switch (Feature)
		{
		case ETargetPlatformFeatures::AudioStreaming:
			return TPlatformProperties::SupportsAudioStreaming();

		case ETargetPlatformFeatures::DistanceFieldShadows:
			return TPlatformProperties::SupportsDistanceFieldShadows();

		case ETargetPlatformFeatures::DistanceFieldAO:
			return TPlatformProperties::SupportsDistanceFieldAO();

		case ETargetPlatformFeatures::GrayscaleSRGB:
			return TPlatformProperties::SupportsGrayscaleSRGB();

		case ETargetPlatformFeatures::HighQualityLightmaps:
			return TPlatformProperties::SupportsHighQualityLightmaps();

		case ETargetPlatformFeatures::LowQualityLightmaps:
			return TPlatformProperties::SupportsLowQualityLightmaps();

		case ETargetPlatformFeatures::MultipleGameInstances:
			return TPlatformProperties::SupportsMultipleGameInstances();

		case ETargetPlatformFeatures::Packaging:
			return false;

		case ETargetPlatformFeatures::CanCookPackages:
			return false;

		case ETargetPlatformFeatures::ShowAggregateDevice:
			return true;

		case ETargetPlatformFeatures::TextureStreaming:
			return TPlatformProperties::SupportsTextureStreaming();
		case ETargetPlatformFeatures::MeshLODStreaming:
			return TPlatformProperties::SupportsMeshLODStreaming();

		case ETargetPlatformFeatures::MemoryMappedFiles:
			return TPlatformProperties::SupportsMemoryMappedFiles();

		case ETargetPlatformFeatures::MemoryMappedAudio:
			return TPlatformProperties::SupportsMemoryMappedAudio();
		case ETargetPlatformFeatures::MemoryMappedAnimation:
			return TPlatformProperties::SupportsMemoryMappedAnimation();

		case ETargetPlatformFeatures::SdkConnectDisconnect:
		case ETargetPlatformFeatures::UserCredentials:
			break;

		case ETargetPlatformFeatures::MobileRendering:
			return false;
		case ETargetPlatformFeatures::DeferredRendering:
			return true;

		case ETargetPlatformFeatures::ShouldSplitPaksIntoSmallerSizes:
			return false;

		case ETargetPlatformFeatures::HalfFloatVertexFormat:
			return true;

		case ETargetPlatformFeatures::LumenGI:
			return TPlatformProperties::SupportsLumenGI();

		case ETargetPlatformFeatures::HardwareLZDecompression:
			return TPlatformProperties::SupportsHardwareLZDecompression();
		}

		return false;
	}
private:
	FString IniPlatformNameValue;
};
