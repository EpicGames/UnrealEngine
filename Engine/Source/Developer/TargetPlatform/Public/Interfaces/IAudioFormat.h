// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Algo/IndexOf.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"

/** 
 * A non-UObject based structure used to pass data about a sound
 * node wave around the engine and tools.
 */
struct FSoundQualityInfo
{
	/** Holds the quality value ranging from 1 [poor] to 100 [very good]. */
	int32 Quality = 0;

	/** Holds the number of distinct audio channels. */
	uint32 NumChannels = 0;

	/** Holds the number of PCM samples per second. */
	uint32 SampleRate = 0;

	/** Holds the size of sample data in bytes. */
	uint32 SampleDataSize = 0;

	/** Holds the length of the sound in seconds. */
	float Duration = 0.f;

	/** Holds whether the sound will be streamed. */
	bool bStreaming = false;

	/** Holds a string for debugging purposes. */
	FString DebugName;
};


/**
 * Interface for audio formats.
 */
class IAudioFormat : public IModularFeature
{
public:

	/**
	 * Returns the name of the modular feature.
	 * @return FName of the feature. To be used to query IModularFeatures
	 */
	FORCEINLINE static FName GetModularFeatureName()
	{
		static const FName Name(TEXT("IAudioFormat"));
		return Name;
	}
	
	
	/**
	 * Checks whether parallel audio cooking is allowed.
	 *
	 * Note: This method is not currently used yet.
	 *
	 * @return true if this audio format can cook in parallel, false otherwise.
	 */
	virtual bool AllowParallelBuild() const
	{
		return false;
	}

	/**
	 * Cooks the source data for the platform and stores the cooked data internally.
	 *
	 * @param Format The desired format.
	 * @param SrcBuffer The source buffer. Buffers are 16 bit PCM, either mono or stereo (check QualityInfo.NumChannels)
	 * @param QualityInfo All the information the compressor needs to compress the audio. QualityInfo.Duration is unset.
	 * @param OutBuffer Will hold the resulting compressed audio.
	 * @return true on success, false otherwise.
	 */
	virtual bool Cook( FName Format, const TArray<uint8>& SrcBuffer, FSoundQualityInfo& QualityInfo, TArray<uint8>& OutBuffer ) const = 0;

	/**
	 * Cooks up to 8 mono files into a multi-stream file (e.g. 5.1). The front left channel is required, the rest are optional.
	 *
	 * @param Format The desired format.
	 * @param SrcBuffers The source buffers. Buffers are mono 16 bit PCM.
	 * @param QualityInfo All the information the compressor needs to compress the audio. QualityInfo.Duration is unset.
	 * @param OutBuffer Will contain the resulting compressed audio.
	 * @return true if succeeded, false otherwise
	 */
	virtual bool CookSurround( FName Format, const TArray<TArray<uint8> >& SrcBuffers, FSoundQualityInfo& QualityInfo, TArray<uint8>& OutBuffer ) const = 0;

	/**
	 * Gets the list of supported formats.
	 *
	 * @param OutFormats Will contain the list of formats.
	 */
	virtual void GetSupportedFormats( TArray<FName>& OutFormats ) const = 0;

	/**
	 * Gets the current version of the specified audio format.
	 *
	 * @param Format The format to get the version for.
	 * @return Version number.
	 */
	virtual uint16 GetVersion( FName Format ) const = 0;

	/** 
	 * Re-compresses raw PCM to the the platform dependent format, and then back to PCM. Used for quality previewing.
	 *
	 * This function is, as far as I can tell, unused.
	 * 
	 * @param Format The desired format.
	 * @param SrcBuffer Uncompressed PCM data.
	 * @param QualityInfo All the information the compressor needs to compress the audio.
	 * @param OutBuffer Uncompressed PCM data after being compressed.
	 * @return The size of the compressed audio, or 0 on failure.
	 */
	virtual int32 Recompress( FName Format, const TArray<uint8>& SrcBuffer, FSoundQualityInfo& QualityInfo, TArray<uint8>& OutBuffer ) const = 0;

	/**
	 * Given the encoded buffer, returns the minimum number of bytes required to perform ICompressedAudioInfo::ReadCompressedInfo() or IStreamedCompressedInfo::ParseHeader() for this file.
	 * 
	 * This is called prior to SplitDataForStreaming(), and is used for determining FirstChunkMaxSize.
	 * 
	 * @param Format the codec that SrcBuffer was compressed as.
	 * @param SrcBuffer the compressed data that will later be split into individual chunks.
	 */
	virtual int32 GetMinimumSizeForInitialChunk(FName Format, const TArray<uint8>& SrcBuffer) const { return 0; }

	/**
	 * Splits compressed data into chunks suitable for streaming audio.
	 *
	 * @param SrcBuffer Pre-compressed data as an array of bytes.
	 * @param OutBuffers Array of buffers that contain the chunks the original data was split into.
	 * @param FirstChunkMaxSize The maximum size for the chunk that will be loaded inline with it's owning USoundWave asset.
	 * @param MaxChunkSize The maximum chunk size for each chunk. The chunks will be zero-padded to match this chunk size in the bulk data serialization.
	 * @return Whether bulk data could be split for streaming.
	 */
	virtual bool SplitDataForStreaming(const TArray<uint8>& SrcBuffer, TArray<TArray<uint8>>& OutBuffers, const int32 FirstChunkMaxSize, const int32 MaxChunkSize) const {return false;}

	virtual bool RequiresStreamingSeekTable() const { return false; }

	struct FSeekTable
	{
		TArray<uint32> Times;			// Times in AudioFrames.
		TArray<uint32> Offsets;			// Offset in the compressed data.
	};	

	/**
	* Extracts the embedded seek-table, removing it, and outputting it separately.
	* NOTE: TArray is modified in place. The seek-table is parsed and removed from it.	
	* @param InOutBuffer Pre-compressed data containing seek-table and compressed data as array of bytes.
	* @param OutSeekTable Seektable in its generic form.
	* @return Success or failure
	*/
	virtual bool ExtractSeekTableForStreaming(TArray<uint8>& InOutBuffer, FSeekTable& OutSeektable) const { return false; }

	/**
	 * Returns true if the target platform name is supported 	 
	 */
	virtual bool IsPlatformSupported(const FName InPlatformName) const { return true; };
	
	/**
	 * Returns true if this format/codec requires hardware. 
	 **/
	virtual bool IsHardwareFormat() const { return false; }

	/**
	 * Returns true if the sample rate is supported by this codec.
	 * @param InSampleRate The Rate to test
	 * @return true or false if supported
	 */
	bool IsSampleRateSupported(const int32 InSampleRate) const
	{
		// Query the supported rates for this Format.
		const TConstArrayView<int32> SupportedRates = GetSupportedSampleRates();
		if (SupportedRates.Num() > 0)
		{
			return Algo::IndexOf(SupportedRates, InSampleRate) != INDEX_NONE;
		}

		// Assume success if no supported rates are defined. 
		return true;
	}

	/**
	 * Returns true if the sample rate is supported by this codec.
	 * @param InChannelCount The channel count to test for
	 * @return true or false if supported
	 */
	bool IsChannelCountSupported(const int32 InChannelCount) const
	{
		// Query the supported counts for this Format.
		const TConstArrayView<int32> SupportedCounts = GetSupportedChannelCounts();
		if (SupportedCounts.Num() > 0)
		{
			return Algo::IndexOf(SupportedCounts, InChannelCount) != INDEX_NONE;
		}

		// Assume success if no supported rates are defined. 
		return true;
	}

	/**
	 * Returns the list of supported sample rates
	 * NOTE: An empty list implies all rates are supported.
	 * @return List of supported sample rate for this format/codec
	 */
 	virtual TConstArrayView<int32> GetSupportedSampleRates() const { return {}; }

	/**
	 * Returns the list of supported channel counts
	 * NOTE: An empty list implies all counts are supported.
	 * @return List of supported channel counts
	 */
	virtual TConstArrayView<int32> GetSupportedChannelCounts() const { return {}; }

	/** Self Registering Constructor */
	IAudioFormat()
    {
    	IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
    }
    
	/** Self Registering Virtual destructor. */
	virtual ~IAudioFormat()
    {
    	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
    }
};

