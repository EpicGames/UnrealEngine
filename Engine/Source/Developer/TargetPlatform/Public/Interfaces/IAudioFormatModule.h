// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class IAudioFormat;

/**
 * Interface for audio format modules.
 */
class IAudioFormatModule
	: public IModuleInterface
{
public:

	/**
	 * Gets the audio format.
	 *
	 * @return The audio format interface.
	 */
	virtual IAudioFormat* GetAudioFormat() = 0;

public:

	/** Virtual destructor. */
	~IAudioFormatModule() = default;
};
