// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "ITargetDevice.h"

class SWidget;

class IDeviceManagerCustomPlatformWidgetCreator
{
public:
	virtual TSharedPtr<SWidget> CreateAddDeviceWidget(const FString& InPlatformName) = 0;
	virtual bool IsAddDeviceInputValid(const FString& InPlatformName, const TSharedPtr<SWidget>& UserData) = 0;

	virtual TSharedPtr<SWidget> CreateDeviceInfoWidget(const FString& InPlatformName, const ITargetDevicePtr& InDevice) = 0;

	virtual void AddDevice(const FString& InPlatformName, const TSharedPtr<SWidget>& UserData) = 0;
};
