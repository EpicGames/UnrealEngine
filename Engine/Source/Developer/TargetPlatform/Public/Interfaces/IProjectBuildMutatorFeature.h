// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Features/IModularFeature.h"

#define PROJECT_BUILD_MUTATOR_FEATURE "ProjectBuildMutatorFeature"

class IProjectBuildMutatorFeature : public IModularFeature
{
public:
	virtual bool RequiresProjectBuild(const FName& InPlatformInfoName, FText& OutReason) const = 0;
};
