// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Wildcard string used to search for shader format modules. */
#define SHADERFORMAT_MODULE_WILDCARD TEXT("*ShaderFormat*")

class FSerializedShaderArchive;
class FShaderPreprocessOutput;
class FCompressedBuffer;
class FSharedBuffer;
struct FShaderCompilerEnvironment;
struct FShaderCompilerInput;
struct FShaderCompilerOutput;
struct FShaderDebugDataContext;

/**
 * IShaderFormat, shader pre-compilation abstraction
 */
class IShaderFormat
{
public:

	/**
	 * Gets the current version of the specified shader format.
	 *
	 * @param Format The format to get the version for.
	 * @return Version number.
	 */
	virtual uint32 GetVersion(FName Format) const = 0;

	/**
	 * Gets the list of supported formats.
	 *
	 * @param OutFormats Will hold the list of formats.
	 */
	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const = 0;
	

	/**
	 * Can this shader format strip shader code for packaging in a shader library?
	 * @param bNative Whether the final shader library uses a native format which may determine if the shader is stripped.
	 * @returns True if and only if the format can strip extraneous data from shaders to be included in a shared library, otherwise false.
	 */
	virtual bool CanStripShaderCode(bool const bNativeFormat) const { return false; }
	
	/**
	 * Strips the shader bytecode provided of any unnecessary optional data elements when archiving shaders into the shared library.
	 *
	 * @param Code The byte code to strip (must be uncompressed).
	 * @param DebugOutputDir The output directory to write the debug symbol file for this shader.
	 * @param bNative Whether the final shader library uses a native format which may determine how the shader is stripped.
	 * @return True if the format has successfully stripped the extraneous data from shaders, otherwise false
	 */
    virtual bool StripShaderCode(TArray<uint8>& Code, FString const& DebugOutputDir, bool const bNative) const { return false; }
    
	/**
	* Whether this shader format supports a format-specific archive for precompiled shader code.
	*
	* @returns true if shader archives are supported, false otherwise.
	*/
	virtual bool SupportsShaderArchives() const { return false; }
	
	/**
     * Create a format specific archive for precompiled shader code.
     *
     * @param LibraryName The name of this shader library.
     * @param Format The format of shaders to cache.
     * @param WorkingDirectory The working directory.
	 * @param The output directory for the archive.
	 * @param The directory for the debug data previously generated.
	 * @param Optional pointer to a TArray that on success will be filled with a list of the written file paths.
     * @returns true if the archive was created
     */
    virtual bool CreateShaderArchive(FString const& LibraryName,
		FName ShaderFormatAndShaderPlatformName,
		const FString& WorkingDirectory,
		const FString& OutputDir,
		const FString& DebugOutputDir,
		const FSerializedShaderArchive& SerializedShaders,
		const TArray<FSharedBuffer>& ShaderCode,
		TArray<FString>* OutputFiles) const
	{ 
		return false; 
	}
	
	/**
	 * Can the shader format compile shaders to the native binary format for the platform.
	 * @returns True if the native compiler is available and configured, otherwise false.
	 */
	virtual bool CanCompileBinaryShaders() const { return true; }

	/**
	* Returns name of directory with platform specific shaders.
	*
	* @returns Name of directory with platform specific shaders.
	*/
	virtual const TCHAR* GetPlatformIncludeDirectory() const = 0;
	

	/**
	 * Called before a shader is compiled to allow the platform shader format to modify the shader compiler input,
	 * e.g. by adding console variable values relevant to shader compilation on that platform.
	 */
	virtual void ModifyShaderCompilerInput(FShaderCompilerInput& Input) const { }

	/**
	 * Called when a shader resource is cooked, so the shader format can perform platform-specific operations on the debug data.
	 * Does nothing on platforms that make no use of the symbol data.
	 */
	virtual void NotifyShaderCompiled(const FCompressedBuffer& SymbolData, FName Format, const FString& DebugInfo = FString()) const { }

	/** Called at the end of a cook to free resources and finalize artifacts created during the cook. */
	virtual void NotifyShaderCompilersShutdown(FName Format) const { }

	/**
	 * Appends shader key text to the provided key string for use in DDC unique key construction.
	 * @param KeyString String that will get shader key text appended to.
	 */
	virtual void AppendToKeyString(FString& KeyString) const { }

	/**
	 * Execute all shader preprocessing steps, storing the output in the PreprocessOutput struct
	 */
	virtual bool PreprocessShader(const FShaderCompilerInput& Input, const FShaderCompilerEnvironment& Environment, FShaderPreprocessOutput& PreprocessOutput) const { return false; };

	/**
	 * Compile the specified preprocessed shader.
	 */
	virtual void CompilePreprocessedShader(
		const FShaderCompilerInput& Input,
		const FShaderPreprocessOutput& PreprocessOutput,
		FShaderCompilerOutput& Output) const {}

	/**
	 * Compile the specified preprocessed shaders; only called if the call to RequiresSecondaryCompile given the first preprocess 
	 * output returns true. The shader system will pack these outputs together in the following format: 
	 * [int32 key][uint32 primary length][uint32 secondary length][full primary shader code][full secondary shader code]
	 * where "key" is the return value of the GetPackedShaderKey function (this should also be implemented by any backends which
	 * require secondary compilation and is used by the RHI to differentiate packed shader code from single shaders)
	 */
	virtual void CompilePreprocessedShader(
		const FShaderCompilerInput& Input,
		const FShaderPreprocessOutput& PrimaryPreprocessOutput,
		const FShaderPreprocessOutput& SecondaryPreprocessOutput,
		FShaderCompilerOutput& PrimaryOutput,
		FShaderCompilerOutput& SecondaryOutput) const {}

	/**
	 * Predicate which should return true if a second preprocess & compilation is required given the initial preprocess output.
	 * This is generally be determined by analyzing the directives on the given preprocess output (which are set by the presence
	 * of UESHADERMETADATA directives in the original source), though other conditions are possible and up to the implementation.
	 *  In the event this returns true, PreprocessShader will be called an additional time, with the bIsSecondary field on the 
	 * FShaderPreprocessOutput set to true, and the overload of CompilePreprocessedShader taking two FShaderPreprocessOutput and 
	 * two FShaderCompilerOutput objects will be called instead of the single object variants. Note that this is done instead of 
	 * calling the single object variant twice so shader formats can statically distinguish primary compilation in a dual-compile 
	 * case from the single-compile case.
	 */
	virtual bool RequiresSecondaryCompile(
		const FShaderCompilerInput& Input,
		const FShaderCompilerEnvironment& Environment,
		const FShaderPreprocessOutput& PreprocessOutput) const { return false; }

	/**
	* For shader formats which support secondary compilation, this can optionally return a name that is appended to diagnostic messages
	* to distinguish errors in the secondary compile from the primary.
	*/
	virtual FStringView GetSecondaryCompileName() const { return TEXTVIEW(""); }

	/**
	 * Virtual function that can be implemented by backends which require secondary compilation to specify a unique "key" identifier
	 * for the packing scheme (for validation in the runtime at load). see RequiresSecondaryCompile and the overloads of 
	 * CompilePreprocessedShader for additional details.
	 */
	virtual int32 GetPackedShaderKey() const { return 0; }

	/* 
	 * Implement to output debug info for a single compile job.
	 * This will be called for all jobs (including those found in the job cache) but only if debug info is enabled for the job.
	 * Note that any debug info output in CompilePreprocessedShader will only be done for the job that actually executes the
	 * compile step, as such any debug outputs that are desirable for all jobs should be written by this function.
	 * A BaseShaderFormat implementation is provided in ShaderCompilerCommon which dumps preprocessed and stripped USFs along
	 * with the hash of the shader code in an OutputHash.txt file; if no additional custom debug output is required the shader 
	 * format can inherit from FBaseShaderFormat instead of IShaderFormat.
	 */
	virtual void OutputDebugData(
		const FShaderCompilerInput& Input,
		const FShaderPreprocessOutput& PreprocessOutput,
		const FShaderCompilerOutput& Output,
		FShaderDebugDataContext& Ctx) const {};

	/*
	 * Implement to output debug info for the case where a secondary preprocessed shader was created. As with the dual-output version
	 * of CompilePreprocessedShader this is provided so shader formats can statically distinguish the single and dual output cases.
	 * Note that BaseShaderFormat does not provide an implementation of this so it needs to be explicitly implemented by backends 
	 * which require secondary compilation, even if inheriting from FBaseShaderFormat.
	 */
	virtual void OutputDebugData(
		const FShaderCompilerInput& Input,
		const FShaderPreprocessOutput& PreprocessOutput,
		const FShaderPreprocessOutput& SecondaryPreprocessOutput,
		const FShaderCompilerOutput& Output,
		const FShaderCompilerOutput& SecondaryOutput,
		FShaderDebugDataContext& Ctx) const {};

	/* Implement to output a minimal set of debug info based solely on the compile input. This is invoked when preprocessing fails. */
	virtual void OutputDebugDataMinimal(const FShaderCompilerInput& Input, FShaderDebugDataContext& Ctx) const {};

public:
	/** Virtual destructor. */
	virtual ~IShaderFormat() = default;
};
