// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

class IShaderFormat;

/**
 * Interface for shader format modules.
 */
class IShaderFormatModule
	: public IModuleInterface
{
public:

	/**
	 * Gets the shader format.
	 *
	 * @return The shader format interface.
	 */
	virtual IShaderFormat* GetShaderFormat() = 0;

public:

	/** Virtual destructor. */
	~IShaderFormatModule() = default;
};
