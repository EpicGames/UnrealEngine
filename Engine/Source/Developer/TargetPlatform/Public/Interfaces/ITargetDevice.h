// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Interfaces/TargetDeviceId.h"
#include "Interfaces/ITargetDeviceSocket.h"

class ITargetDevice;
class ITargetDeviceOutput;
class ITargetPlatform;

/**
 * Enumerates features that may be supported by target devices.
 */
enum class ETargetDeviceFeatures
{
	/** Multiple instances of a game can run at the same time. */
	MultiLaunch,

	/** The device can be powered off remotely. */
	PowerOff,

	/** The device can be powered on remotely. */	  
	PowerOn,

	/** Snapshot of processes running on the device. */
	ProcessSnapshot,

	/** The device can be rebooted remotely. */
	Reboot
};


/**
 * Enumerates target device types.
 */
enum class ETargetDeviceTypes
{
	/** Indeterminate device type. */
	Indeterminate,

	/** The device is a web browser (i.e. Flash). */
	Browser,

	/** The device is a game console. */
	Console,

	/** The device is a desktop computer. */
	Desktop,

	/** The device is a smart phone. */
	Phone,

	/** The device is a tablet computer. */
	Tablet,

	/** The device is a standalone HMD */
	HMD
};

/**
 * Enumerates how the target device is connected
 */
enum class ETargetDeviceConnectionTypes
{
	/** It's unknown how the device is connected. */
	Unknown,

	/** The device is connected through USB. */
	USB,

	/** The device is connected via Wifi. */
	Wifi,
	
	/** The device is connected via Ethernet. */
	Ethernet,

	/** The device is running as a simulator on the current Editor. */
	Simulator,

	/** The device is connected via a proprietary connection. */
	Proprietary
};

namespace TargetDeviceTypes
{
	/**
	 * Returns the string representation of the specified ETargetDeviceTypes value.
	 *
	 * @param DeviceType The value to get the string for.
	 * @return A string value.
	 */
	inline FString ToString(ETargetDeviceTypes DeviceType)
	{
		switch (DeviceType)
		{
		case ETargetDeviceTypes::Browser:
			return FString("Browser");

		case ETargetDeviceTypes::Console:
			return FString("Console");

		case ETargetDeviceTypes::Desktop:
			return FString("Desktop");

		case ETargetDeviceTypes::Phone:
			return FString("Phone");

		case ETargetDeviceTypes::Tablet:
			return FString("Tablet");

		case ETargetDeviceTypes::HMD:
			return FString("HMD");

		case ETargetDeviceTypes::Indeterminate:
		default:
			return FString("Indeterminate");
		}
	}
}

namespace TargetDeviceConnectionTypes
{
	/**
	 * Returns the string representation of the specified ETargetDeviceConnectionTypes value.
	 *
	 * @param DeviceConnectionType The value to get the string for.
	 * @return A string value.
	 */
	inline FString ToString(ETargetDeviceConnectionTypes DeviceConnectionType)
	{
		switch (DeviceConnectionType)
		{
		case ETargetDeviceConnectionTypes::USB:
			return FString("USB");

		case ETargetDeviceConnectionTypes::Wifi:
			return FString("Wifi");

		case ETargetDeviceConnectionTypes::Ethernet:
			return FString("Ethernet");

		case ETargetDeviceConnectionTypes::Simulator:
			return FString("Simulator");

		case ETargetDeviceConnectionTypes::Proprietary:
			return FString("Proprietary");

		case ETargetDeviceConnectionTypes::Unknown:
		default:
			return FString("Unknown");
		}
	}
}

/**
 * Enumerates states of threads running on a target device.
 */
enum class ETargetDeviceThreadStates
{
	/** Unknown thread state. */
	Unknown,

	/** The thread can run, but is not running right now. */
	CanRun,

	/** The thread is inactive, i.e. has just been created or exited. */
	Inactive,

	/** The thread cannot run right now. */
	Inhibited,

	/** The thread is in the run queue. */
	RunQueue,

	/** The thread is running. */
	Running
};


/**
 * Enumerates wait states of threads running on a target device.
 */
enum class ETargetDeviceThreadWaitStates
{
	/** Unknown wait state. */
	Unknown,

	/** The thread is blocked by a lock. */
	Locked,
	
	/** The thread is sleeping. */
	Sleeping,

	/** The thread is suspended. */
	Suspended,

	/** The thread is swapped. */
	Swapped,

	/** The thread is waiting on an interrupt. */
	Waiting
};


/**
 * Structure for thread information.
 */
struct FTargetDeviceThreadInfo
{
	/** Holds the exit code. */
	uint64 ExitCode;

	/** Holds the thread identifier. */
	uint32 Id;

	/** Holds the name of the thread. */
	FString Name;

	/** Holds the thread's stack size. */
	uint64 StackSize;

	/** Holds the thread's current state. */
	ETargetDeviceThreadStates State;

	/** Holds the thread's current wait state. */
	ETargetDeviceThreadWaitStates WaitState;
};


/**
 * Structure for information for processes that are running on a target device.
 */
struct FTargetDeviceProcessInfo
{
	/** Holds the process identifier. */
	int64 Id;

	/** Holds the process name. */
	FString Name;

	/** Holds the identifier of the parent process. */
	uint64 ParentId;

	/** Holds the collection of threads that belong to this process. */
	TArray<FTargetDeviceThreadInfo> Threads;

	/** The name of the user that owns this process. */
	FString UserName;
};


/** Type definition for shared pointers to instances of ITargetDevice. */
typedef TSharedPtr<class ITargetDevice, ESPMode::ThreadSafe> ITargetDevicePtr;

/** Type definition for shared references to instances of ITargetDevice. */
typedef TSharedRef<class ITargetDevice, ESPMode::ThreadSafe> ITargetDeviceRef;

/** Type definition for weak pointers to instances of ITargetDevice. */
typedef TWeakPtr<class ITargetDevice, ESPMode::ThreadSafe> ITargetDeviceWeakPtr;

/** Type definition for shared pointers to instances of ITargetDeviceOutput. */
typedef TSharedPtr<class ITargetDeviceOutput, ESPMode::ThreadSafe> ITargetDeviceOutputPtr;

/**
 * Interface for target devices.
 */
class ITargetDevice
{
public:

	/**
	 * Connect to the physical device.
	 *
	 * @return true if the device is connected, false otherwise.
	 */
	virtual bool Connect() = 0;

	/**
	 * Disconnect from the physical device.
	 */
	virtual void Disconnect() = 0;

	/**
	 * Gets the device type.
	 *
	 * @return Device type.
	 */
	virtual ETargetDeviceTypes GetDeviceType() const = 0;

	/**
	 * Gets the device model identifier.
	 *
	 * @return ModelId.
	 */
	virtual FString GetModelId() const {return FString();};

	/**
	 * Gets the device OS Version.
	 *
	 * @return OSVersion.
	 */
	virtual FString GetOSVersion() const {return FString();};

	/**
	 * Gets the device architecture.
	 *
	 * @return Device Architecture.
	 */
	virtual FStringView GetArchitecture() const
	{
		return {};
	}
	
	/**
	 * Gets the device connection type.
	 * 
	 * @return Device connection type.
	 */
	virtual ETargetDeviceConnectionTypes GetDeviceConnectionType() const {return ETargetDeviceConnectionTypes::Unknown; };

	/**
	 * Gets the unique device identifier.
	 *
	 * @return Device identifier.
	 * @see GetName
	 */
	virtual FTargetDeviceId GetId() const = 0;

	/**
	 * Gets the name of the device.
	 *
	 * In contrast to GetId(), this method is intended to return a human readable
	 * name for use in the user interface. Depending on the target platform, this
	 * name may be some user defined string, a host name, an IP address, or some
	 * other string identifying the device that does not need to be unique.
	 *
	 * @return Device name.
	 * @see GetId
	 */
	virtual FString GetName() const = 0;

	/**
	 * Gets the name of the operating system running on this device.
	 *
	 * @return Operating system name.
	 */
	virtual FString GetOperatingSystemName() = 0;

	/**
	 * Creates a snapshot of processes currently running on the device.
	 *
	 * @param OutProcessInfos Will contain the information for running processes.
	 * @return The number of returned processes.
	 */
	virtual int32 GetProcessSnapshot( TArray<FTargetDeviceProcessInfo>& OutProcessInfos ) = 0;

	/**
	 * Creates a snapshot of processes currently running on the device.
	 *
	 * @param OutProcessInfos Will contain the information for running processes.
	 * @param CompleteHandler will be invoked when process snapshot information is available
	 * @return True if retrieving a process snapshot is supported
	 */
	virtual bool GetProcessSnapshotAsync(TFunction<void(const TArray<FTargetDeviceProcessInfo>&)> CompleteHandler)
	{
		TArray<FTargetDeviceProcessInfo> InProcessInfos;
		GetProcessSnapshot(InProcessInfos);
		CompleteHandler(InProcessInfos);
		return true;
	}

	/**
	 * Gets the TargetPlatform that this device belongs to.
	 */
protected:
	// Temporary until we get rid of this.
	virtual const class ITargetPlatform& GetTargetPlatform() const
	{
		unimplemented();
		static ITargetPlatform* DummyReference = nullptr;
		// Code should never reach this point, this is just temporary until we port all platforms to new system
		return *DummyReference;
	};
public:

	TARGETPLATFORM_API virtual const class ITargetPlatformSettings& GetPlatformSettings() const;
	TARGETPLATFORM_API virtual const class ITargetPlatformControls& GetPlatformControls() const;

	/**
	 * Checks whether this device is connected.
	 *
	 * @return true if the device is connected, false otherwise.
	 */
	virtual bool IsConnected() = 0;

	/**
	 * Checks whether this is the default device.
	 *
	 * Note that not all platforms may have a notion of default devices.
	 *
	 * @return true if this is the default device, false otherwise.
	 */
	virtual bool IsDefault() const = 0;

	/**
	* Checks whether this device is authorized to be used with this computer.
	*
	* This is true for most platforms by default, but may be false for mobile platforms
	*
	* @return true if this this device is authorized for launch
	*/
	virtual bool IsAuthorized() const { return true;  }

	/**
	 * Powers off the device.
	 *
	 * @param Force Whether to force powering off.
	 * @return true if the device will be powered off, false otherwise.
	 */
	virtual bool PowerOff( bool Force ) = 0;

	/**
	 * Powers on the device.
	 *
	 * @return true if the device will be powered on, false otherwise.
	 */
	virtual bool PowerOn() = 0;

	/** 
	 * Reboot the device.
	 *
	 * @param bReconnect If true, wait and reconnect when done.
	 * @return true if the reboot was successful from the perspective of the PC .
	 */
	virtual bool Reboot( bool bReconnect = false ) = 0;

	/**
	 * Checks whether the target device supports the specified feature.
	 *
	 * @param Feature The feature to check.
	 * @return true if the feature is supported, false otherwise.
	 */
	virtual bool SupportsFeature( ETargetDeviceFeatures Feature ) const = 0;

	/**
	 * Terminates a process that was launched on the device using the Launch() or Run() methods.
	 *
	 * @param ProcessId The identifier of the process to terminate.
	 * @return true if the process was terminated, false otherwise.
	 */
	virtual bool TerminateProcess( const int64 ProcessId ) = 0;

	/**
	 * Set credentials for the user account to use on the device
	 * 
	 * @param UserName The user account on the device we will run under
	 * @param UserPassword The password for the user account on the device we will run under.
	 */
	virtual void SetUserCredentials( const FString& UserName, const FString& UserPassword ) = 0;

	/**
	 * Get credentials for the user account to use on the device
	 * 
	 * @param OutUserName The user account on the device we will run under
	 * @param OutUserPassword The password for the user account on the device we will run under
	 * @return true on success, false if not supported.
	 */
	virtual bool GetUserCredentials( FString& OutUserName, FString& OutUserPassword ) = 0;

	/**
	 * Execute console command on the device
	 * 
	 * @param ExecCommand Command to execute
	 */
	virtual void ExecuteConsoleCommand(const FString& ExecCommand) const {};

	/**
	 * Create device output router
	 *
	 * This will route device logs into specified OutputDevice 
	 * until connection to device is alive
	 *
	 * @param Output OutputDevice to where output should be routed (has to be thread safe)
	 * @return Valid router object for devices that support output routing
	 */
	virtual ITargetDeviceOutputPtr CreateDeviceOutputRouter(FOutputDevice* Output) const { return nullptr; };

	/**
	 * Cancel the application running on the device
	 * @param ProcessIdentifier The bundle id
	 */
	virtual bool TerminateLaunchedProcess(const FString & ProcessIdentifier) { return false;  };
	
	/**
	 * Execute console command on the device to reload the global shader map
	 * 
	 * @param GlobalShaderMapDirectory path to directory that contains global shader map
	 */
	virtual void ReloadGlobalShadersMap(const FString& GlobalShaderMapDirectory) const
	{
		return ExecuteConsoleCommand(TEXT("ReloadGlobalShaders"));
	}

	/**
	 * Opens a direct connection with the device allowing data exchange with a process running on the target.
	 * 
	 * ProtocolIndex is a number that identifies the connection and has to be know both to the target
	 * process and the PC. There can be only one connection using a given protocol index at a time.
	 * 
	 * You may check EHostProtocol enumeration for known protocols used by the engine.
	 * 
	 * This function just opens a communication channel but doesn't check if there is a peer
	 * on the other end of the connection. To know this, you should either check IsProtocolAvailable
	 * or ITargetDeviceSocket::Connected.
	 * 
	 * @param ProtocolIndex Unique index of the communication channel (from 0 to a platform-dependent maximum).
	 * @return Socket object on success, nullptr otherwise.
	 * @see CloseSocket, EHostProtocol, IsProtocolAvailable, ITargetDeviceSocket::Connected
	 */
	virtual ITargetDeviceSocketPtr OpenConnection(uint32 ProtocolIndex)	{ return nullptr; }

	/**
	 * Closes a previously opened connection with a process on the target.
	 * 
	 * @param Socket Socket object returned by OpenSocket.
	 * @see OpenSocket
	 */
	virtual void CloseConnection(ITargetDeviceSocketPtr Socket)			{ }

	/**
	 * Checks if connections using the given ProtocolIndex are available for this device at the moment.
	 * If this function returns true, it means that OpenConnection should succeed and communication
	 * with a game running on the target should be possible.
	 * 
	 * @param ProtocolIndex Unique index of the communication channel (from 0 to a platform-dependent maximum).
	 * @return True if the protocol is available and we can connect to the device, false otherwise.
	 */
	virtual bool IsProtocolAvailable(uint32 ProtocolIndex) const		{ return false; }

public:

	/**
	* Get the "All devices" flag for the platform
	*
	* @return true if the platform has an "All devices" proxy.
	*/
	virtual bool IsPlatformAggregated() const { return false; }

	/**
	* Get the "All devices" name
	*
	* @return "All devices" name.
	*/
	virtual FString GetAllDevicesName() const { return FString(); }

	/**
	* Get the "All devices" default variant
	*
	* @return "All devices" default variant.
	*/
	virtual FName GetAllDevicesDefaultVariant() const { return FName(); }

public:
	
	/** Virtual destructor. */
	virtual ~ITargetDevice() = default;
};
