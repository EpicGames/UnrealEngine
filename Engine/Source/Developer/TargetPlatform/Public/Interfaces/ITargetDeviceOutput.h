// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Interface for target device output log reader.
 */
class ITargetDeviceOutput
{
public:
	/** Virtual destructor. */
	virtual ~ITargetDeviceOutput() = default;
};
