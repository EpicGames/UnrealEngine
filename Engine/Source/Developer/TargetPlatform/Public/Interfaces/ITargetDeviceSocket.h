// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Templates/SharedPointer.h"

/**
 * Interface for target device sockets.
 *
 * This interface provides an abstraction for communicating with processes running on the target.
 */
class ITargetDeviceSocket
{
public:

	/**
	 * Send data to a connected process on the target device.
	 * 
	 * This is a blocking operation and it will return only after the whole buffer
	 * has been sent or an error occurs (e.g. the target disconnects).
	 * 
	 * False return value typically indicates that the remote peer has closed the connection 
	 * on their side. If it happens, this socket should be closed as well.
	 *
	 * @param Data        Buffer containing data to be sent (.
	 * @param BytesToSend The number of bytes from Data that are to be sent.
	 * @return true if communication was successful, false otherwise.
	 * 
	 * @see IHostDevice::OpenConnection, IHostDevice::CloseConnection
	 */
	virtual bool Send(const void* Data, uint64 BytesToSend) = 0;

	/**
	 * Receive data from a connected process on the target device.
	 * 
	 * This is a blocking operation and it will return only after the expected amount
	 * of data has been received or an error occurs (e.g. the target disconnects).
	 * 
	 * False return value typically indicates that the remote peer has closed the connection
	 * on their side. If it happens, this socket should be closed as well.
	 *
	 * @param Data           Target buffer for the data to receive (it has to be large enough to store BytesToReceive).
	 * @param BytesToReceive The number of bytes to receive and store in Data.
	 * @return true if communication was successful, false otherwise.
	 * 
	 * @see IHostDevice::OpenConnection, IHostDevice::CloseConnection
	 */
	virtual bool Receive(void* Data, uint64 BytesToReceive) = 0;

	/**
	 * Returns true if this socket is actually connected to another peer and is ready to send/receive data.
	 */
	virtual bool Connected() const = 0;

public:

	/** Virtual destructor. */
	virtual ~ITargetDeviceSocket() = default;

};

// Type definition for shared references to instances of IPlatformHostSocket.
typedef TSharedRef<ITargetDeviceSocket, ESPMode::ThreadSafe> ITargetDeviceSocketRef;

// Type definition for shared pointers to instances of IPlatformHostSocket.
typedef TSharedPtr<ITargetDeviceSocket, ESPMode::ThreadSafe> ITargetDeviceSocketPtr;
