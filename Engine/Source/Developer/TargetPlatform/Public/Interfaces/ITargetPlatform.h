// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "HAL/PlatformMisc.h"
#include "Interfaces/ITargetDevice.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Interfaces/ITargetPlatformSettings.h"
#include "Interfaces/ITargetPlatformControls.h"

class FConfigCacheIni;
class IDeviceManagerCustomPlatformWidgetCreator;
class IPlugin;
struct FDataDrivenPlatformInfo;

namespace PlatformInfo
{
	// Forward declare type from DesktopPlatform rather than add an include dependency to everything using ITargetPlatform
	struct FTargetPlatformInfo;
}


/**
 * Interface for target platforms.
 *
 * This interface is only left for compatibility with platforms that have not been updated to TargetPlatformSettings and TargetPlatformControls
 * Please only add new functions to ITargetPlatformSettings(if the sdk is not required) or ITargetPlatformControls(if the sdk is required)
 */
class ITargetPlatform : public ITargetPlatformSettings, public ITargetPlatformControls
{
public:
	const ITargetPlatformControls* GetTargetPlatformControls() const  { return this; }

public:
	ITargetPlatform() :ITargetPlatformControls(this){}
	/**
	 * Given a platform ordinal number, returns the corresponding ITargetPlatform instance
	 */
	TARGETPLATFORM_API static const ITargetPlatform* GetPlatformFromOrdinal(int32 Ordinal);
	/**
	 * Gets an event delegate that is executed when a new target device has been discovered.
	 */
	DECLARE_EVENT_OneParam(ITargetPlatform, FOnTargetDeviceDiscovered, ITargetDeviceRef /*DiscoveredDevice*/);
	UE_DEPRECATED(5.4, "ITargetPlatform::OnDeviceDiscovered is deprecated, use ITargetPlatformControls::OnDeviceDiscovered instead")
	static TARGETPLATFORM_API FOnTargetDeviceDiscovered& OnDeviceDiscovered();

	/**
	 * Gets an event delegate that is executed when a target device has been lost, i.e. disconnected or timed out.
	 */
	DECLARE_EVENT_OneParam(ITargetPlatform, FOnTargetDeviceLost, ITargetDeviceRef /*LostDevice*/);
	UE_DEPRECATED(5.4, "ITargetPlatform::OnDeviceLost is deprecated, use ITargetPlatformControls::OnDeviceLost instead")
	static TARGETPLATFORM_API FOnTargetDeviceLost& OnDeviceLost();

public:

	/** Virtual destructor. */
	virtual ~ITargetPlatform() = default;

protected:
	static TARGETPLATFORM_API int32 AssignPlatformOrdinal(const ITargetPlatform& Platform);
};

/**
 * Target platform identifier
 *
 * This is really just a wrapper around an integer ordinal value, to prevent
 * accidental mix-ups with other classes of integers. It also provides more
 * context to a reader of the code.
 *
 * @see ITargetPlatform::GetPlatformOrdinal()
 */

struct FTargetPlatform
{
	inline FTargetPlatform(const ITargetPlatform& Platform)
	{
		Ordinal = Platform.GetPlatformOrdinal();
	}

	inline uint32 GetOrdinal() const { return Ordinal;  }

	bool operator<(const FTargetPlatform& Other) const
	{
		return Ordinal < Other.Ordinal;
	}

	bool operator==(const FTargetPlatform& Other) const
	{
		return Ordinal == Other.Ordinal;
	}

	friend inline uint32 GetTypeHash(const FTargetPlatform& Key) { return Key.GetOrdinal(); }

private:
	uint32 Ordinal = 0;
};

/**
 * Target platform set implementation using bitmask for compactness
 */
class FTargetPlatformSet
{
public:
	inline void Add(const FTargetPlatform& Platform)
	{
		const uint32 Ordinal = Platform.GetOrdinal();

		Mask |= 1ull << Ordinal;
	}

	inline void Remove(const FTargetPlatform& Platform)
	{
		const uint32 Ordinal = Platform.GetOrdinal();

		Mask &= ~(1ull << Ordinal);
	}

	/// Remove all members from the Platforms set from this set
	inline void Remove(const FTargetPlatformSet& Platforms)
	{
		Mask &= ~Platforms.Mask;
	}

	/// Check if this set contains any of the members of the Other set
	inline bool Contains(const FTargetPlatform& Platform) const
	{
		const uint32 Ordinal = Platform.GetOrdinal();

		return !!(Mask & (1ull << Ordinal));
	}

	/// Check if this set contains any of the members of the Other set
	bool ContainsAny(const FTargetPlatformSet& Other) const
	{
		return !!(Mask & Other.Mask);
	}

	bool IsEmpty() const
	{
		return Mask == 0;
	}

	void Merge(const FTargetPlatformSet& Other)
	{
		Mask |= Other.Mask;
	}

	void Clear()
	{
		Mask = 0;
	}

	bool operator==(const FTargetPlatformSet& Other) const
	{
		return this->Mask == Other.Mask;
	}

	uint32 GetHash() const
	{
		// This may not be awesome but I don't know that it's an actual problem in practice
		return uint32(this->Mask ^ (this->Mask >> 32));
	}

	/**
	 * Iterate over all set members
	 *
	 * @param Callback - callback accepting a const ITargetPlatform* argument
	 */
	template<typename Func>
	void ForEach(Func&& Callback) const
	{
		// This could maybe be smarter and leverage intrinsics to directly 
		// scan for the next set bit but I just want to make it work for 
		// now so let's keep it simple until it shows up in a profile

		uint64 IterMask = Mask;
		int Ordinal = 0;

		while(IterMask)
		{
			if (IterMask & 1)
			{
				Callback(ITargetPlatform::GetPlatformFromOrdinal(Ordinal));
			}

			IterMask >>= 1;
			++Ordinal;
		}
	}

	// TODO: support for ranged for? It's easy to make mistakes when refactoring
	// regular for loops into ForEach type constructs since the semantics of any
	// return statements does change when you put them inside a lambda.

private:
	uint64	Mask = 0;
};
