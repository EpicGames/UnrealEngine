// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Interfaces/ITargetDevice.h"

class IDeviceManagerCustomPlatformWidgetCreator;
class IPlugin;
struct FDataDrivenPlatformInfo;
class ITargetPlatformSettings;
namespace PlatformInfo
{
	// Forward declare type from DesktopPlatform rather than add an include dependency to everything using ITargetPlatform
	struct FTargetPlatformInfo;
}


enum class EPlatformAuthentication
{
	Never,
	Possible,
	Always,
};

/**
 * Flags specifying what is needed to be able to complete and deploy a build.
 */
namespace ETargetPlatformReadyStatus
{
	/** Ready */
	inline const int32 Ready = 0;

	/** SDK Not Found*/
	inline const int32 SDKNotFound = 1;

	/** Code Build Not Supported */
	inline const int32 CodeUnsupported = 2;

	/** Plugins Not Supported */
	inline const int32 PluginsUnsupported = 4;

	/** Signing Key Not Found */
	inline const int32 SigningKeyNotFound = 8;

	/** Provision Not Found */
	inline const int32 ProvisionNotFound = 16;

	/** Manifest Not Found */
	inline const int32 ManifestNotFound = 32;

	/** Remote Server Name Empty */
	inline const int32 RemoveServerNameEmpty = 64;

	/** License Not Accepted  */
	inline const int32 LicenseNotAccepted = 128;

	/** Code Build Required */
	inline const int32 CodeBuildRequired = 256;
};


class ITargetPlatformControls
{
public:
	ITargetPlatformControls(ITargetPlatformSettings* InTargetPlatformSettings) :TargetPlatformSettings(InTargetPlatformSettings) {};
	/**
	 * Add a target device by name.
	 *
	 * @param DeviceName The name of the device to add.
	 * @param bDefault Whether the added device should be the default.
	 * @return true if the device was added, false otherwise.
	 */
	virtual bool AddDevice(const FString& DeviceName, bool bDefault) = 0;

	/**
	 * Add a target device.
	 *
	 * @param DeviceId The id of the device to add.
	 * @param DeviceUserFriendlyName The user friendly name of the device to add.
	 * @param Username The username for the device to add.
	 * @param Password The password for the device to add.
	 * @param bDefault Whether the added device should be the default.
	 * @return true if the device was added, false otherwise.
	 */
	virtual bool AddDevice(const FString& DeviceId, const FString& DeviceUserFriendlyName, const FString& Username, const FString& Password, bool bDefault) = 0;

	/**
	* Returns the name of this platform
	*
	* @return Platform name.
	* @see DisplayName
	*/
	virtual FString PlatformName() const = 0;

	/**
	 * Gets the platform's display name.
	 *
	 * @see PlatformName
	 */
	virtual FText DisplayName() const = 0;

	/**
	 * Checks whether the platform's build requirements are met so that we can do things like package for the platform.
	 *
	 * @param bProjectHasCode true if the project has code, and therefore any compilation based SDK requirements should be checked.
	 * @param Configuration The configuration being built
	 * @param bRequiresAssetNativization Whether asset nativization is required
	 * @param OutTutorialPath Let's the platform tell the editor a path to show some information about how to fix any problem.
	 * @param OutDocumentationPath Let's the platform tell the editor a documentation path.
	 * @param CustomizedLogMessage Let's the platform return a customized log message instead of the default for the returned status.
	 * @return A mask of ETargetPlatformReadyStatus flags to indicate missing requirements, or 0 if all requirements are met.
	 */
	virtual int32 CheckRequirements(bool bProjectHasCode, EBuildConfiguration Configuration, bool bRequiresAssetNativization, FString& OutTutorialPath, FString& OutDocumentationPath, FText& CustomizedLogMessage) const = 0;

	/**
	 * Checks whether the current project needs a temporary .target.cs file to be packaged as a code project.
	 *
	 * @param bProjectHasCode Whether the project has code
	 * @param Configuration The configuration being built
	 * @param bRequiresAssetNativization Whether asset nativization is enabled
	 * @param OutReason On success, includes a description of the reason why a target is required
	 * @return True if a temporary target is required
	 */
	virtual bool RequiresTempTarget(bool bProjectHasCode, EBuildConfiguration Configuration, bool bRequiresAssetNativization, FText& OutReason) const = 0;

	/**
	 * Returns the information about this target platform
	 */
	virtual const PlatformInfo::FTargetPlatformInfo& GetTargetPlatformInfo() const = 0;

	/**
	 * Returns the information about the platform as a whole
	 */
	virtual const FDataDrivenPlatformInfo& GetPlatformInfo() const = 0;

	/**
	 * Gets the name of the device profile to use when cooking this TargetPlatform
	 */
	virtual FString CookingDeviceProfileName() const = 0;

	/**
	 * Enables/Disable the device check
	 */
	virtual void EnableDeviceCheck(bool OnOff) = 0;

	/**
	 * Returns all discoverable physical devices.
	 *
	 * @param OutDevices Will contain a list of discovered devices.
	 */
	virtual void GetAllDevices(TArray<ITargetDevicePtr>& OutDevices) const = 0;
	
	/**
	 * Gets a new compression format to use in place of Zlib. This should be rarely implemented
	 *
	 * @return Compression format to use instead of Zlib
	 */
	virtual FName GetZlibReplacementFormat() const = 0;

	/**
	 * Gets the alignment of memory mapping for this platform, typically the page size.
	 *
	 * @return alignment of memory mapping.
	 */
	virtual int64 GetMemoryMappingAlignment() const = 0;
	
	/**
	 * Generates a platform specific asset manifest given an array of FAssetData.
	 *
	 * @param PakchunkMap A map of asset path to Pakchunk file indices for all of the assets.
	 * @param PakchunkIndicesInUse A set of all Pakchunk file indices used by this set of assets.
	 * @return true if the manifest was successfully generated, or if the platform doesn't need a manifest .
	 */
	virtual bool GenerateStreamingInstallManifest(const TMultiMap<FString, int32>& PakchunkMap, const TSet<int32>& PakchunkIndicesInUse) const = 0;

	/**
	 * Gets the default device.
	 *
	 * Note that not all platforms may have a notion of default devices.
	 *
	 * @return Default device.
	 */
	virtual ITargetDevicePtr GetDefaultDevice() const = 0;

	/**
	 * Gets an interface to the specified device.
	 *
	 * @param DeviceId The identifier of the device to get.
	 * @return The target device (can be nullptr).
	 */
	virtual ITargetDevicePtr GetDevice(const FTargetDeviceId& DeviceId) = 0;

	/**
	 * Checks whether this platform has only editor data (typically desktop platforms).
	 *
	 * @return true if this platform has editor only data, false otherwise.
	 */
	virtual bool HasEditorOnlyData() const = 0;

	/**
	 * Checks whether this platform will allow editor objects to be cooked, as opposed to editoronly properties. This will allow a
	 * target platform to cook editoronly objects, but as if they were being cooked for a client. This is useful for a cooked editor
	 * scenario, where every pacakge is cooked, editor and game alike.
	 *
	 * @return true if this platform allows editor objects to be cooked, false otherwise.
	 */
	virtual bool AllowsEditorObjects() const = 0;

	/**
	 * Checks whether this platform will allow development objects to be cooked. This is separate from AllowsEditorObjects
	 * because cooked editors can be shipped with editor objects but still need to remove development assets.
	 *
	 * @return true if this platform allows development objects to be cooked, false otherwise.
	 */
	virtual bool AllowsDevelopmentObjects() const = 0;

	/**
	 * Checks whether this platform is only a client (and must connect to a server to run).
	 *
	 * @return true if this platform must connect to a server.
	 */
	virtual bool IsClientOnly() const = 0;

	/**
	 * Checks whether this platform is little endian.
	 *
	 * @return true if this platform is little-endian, false otherwise.
	 */
	virtual bool IsLittleEndian() const = 0;

	/**
	 * Checks whether this platform is only a server.
	 *
	 * @return true if this platform has no graphics or audio, etc, false otherwise.
	 */
	virtual bool IsServerOnly() const = 0;

	/**
	 * Checks whether this platform is enabled for the given plugin in the currently active project.
	 */
	virtual bool IsEnabledForPlugin(const IPlugin& Plugin) const = 0;

	/**
	* Checks whether this platform supports shader compilation over XGE interface.
	*
	* @return true if this platform can distribute shader compilation threads with XGE.
	*/
	virtual bool CanSupportRemoteShaderCompile() const = 0;

	/**
	* Provide platform specific file dependency patterns for SN-DBS shader compilation.
	*
	* @param OutDependencies Platform specific dependency file patterns are uniquely appended to this array.
	*/
	virtual void GetShaderCompilerDependencies(TArray<FString>& OutDependencies) const = 0;

	/**
	 * Checks whether the platform's SDK requirements are met so that we can do things like
	 * package for the platform
	 *
	 * @param bProjectHasCode true if the project has code, and therefore any compilation based SDK requirements should be checked
	 * @param OutDocumentationPath Let's the platform tell the editor a path to show some documentation about how to set up the SDK
	 * @return true if the platform is ready for use
	 */
	virtual bool IsSdkInstalled(bool bProjectHasCode, FString& OutDocumentationPath) const = 0;

	/**
	 * Checks whether this platform requires cooked data (typically console platforms).
	 *
	 * @return true if this platform requires cooked data, false otherwise.
	 */
	virtual bool RequiresCookedData() const = 0;

	/**
	 * Checks whether this platform requires the originally released version (in addition to the
	 * previously released version) to create a patch
	 *
	 * @return true if this platform requires the originally released version, false otherwise.
	 */
	virtual bool RequiresOriginalReleaseVersionForPatch() const = 0;

	/**
	 * Checks whether the platform has non-trivial patching requirements, where regular workflows
	 * may result in invalid patches. It is recommended to check the platform's documentation.
	 *
	 * @return true if this platform has non-trivial patching requirements, false otherwise.
	 */
	virtual bool HasNonTrivialPatchGeneration() const = 0;

	/**
	* Checks whether this platform has a secure shippable package format, and therefore doesn't need any encryption or signing support
	*
	* @return true if this platform has a secure shippable package format, false otherwise.
	*/
	virtual bool HasSecurePackageFormat() const = 0;
	
	/**
	 * Checks whether this platform requires user credentials (typically server platforms).
	 *
	 * @return enum if this platform requires user credentials.
	 */
	virtual EPlatformAuthentication RequiresUserCredentials() const = 0;

	/**
	 * Returns true if the platform supports the AutoSDK system
	 */
	virtual bool SupportsAutoSDK() const = 0;

	/**
	 * Checks whether this platform supports the specified build target, i.e. Game or Editor.
	 *
	 * @param TargetType The target type to check.
	 * @return true if the build target is supported, false otherwise.
	 */
	virtual bool SupportsBuildTarget(EBuildTargetType TargetType) const = 0;

	/**
	 * Return the TargetType this platform uses at runtime.
	 * Some TargetPlatforms like CookedEditors need to cook with one type, like Client, but then will run as an Editor.
	 * Decisions made based solely on the cook type may cause data mismatches if the runtime type is different.
	 * This is also useful for knowing what plugins will be enabled at runtime.
	 */
	virtual EBuildTargetType GetRuntimePlatformType() const = 0;
	
	/**
	 * Gather per-project cook/package analytics
	 */
	virtual void GetPlatformSpecificProjectAnalytics(TArray<struct FAnalyticsEventAttribute>& AnalyticsParamArray) const = 0;

	/**
	 * Manually refresh the devices
	 *
	 */
	virtual void RefreshAllDevices(){}

#if WITH_ENGINE
	/**
	 * Gets the format to use for a particular body setup.
	 *
	 * @return Physics format.
	 */
	virtual FName GetPhysicsFormat(class UBodySetup* Body) const = 0;

	/**
	 * Gets the texture format to use for each layer in the given texture, for each of the platform's formats.
	 * _Most_ platforms only supply one format for a given texture, so OutFormats.Num() is usually 1. The exception is Android_Multi,
	 * where you can get several formats due to targeting different devices.
	 * OutFormats.Num() == NumberOfPlatformFormats
	 * OutFormats[0...N].Num() == Texture->Source.GetNumLayers().
	 */
	virtual void GetTextureFormats(const class UTexture* Texture, TArray< TArray<FName> >& OutFormats) const = 0;

	/**
	 * Get the limitations on texture size (bytes) on this platform
	 * this limitation is in addition to any dimension limits (specified in device profiles)
	 * all built textures must be <= this size
	 */
	virtual void GetTextureSizeLimits(uint64 & OutMaximumSurfaceBytes, uint64 & OutMaximumPackageBytes) const = 0;

	/**
	 * Gets the texture formats this platform can use
	 *
	 * @param OutFormats will contain all the texture formats which are possible for this platform
	 */
	UE_DEPRECATED(5.9, "GetAllTextureFormats is deprecated.")
	virtual void GetAllTextureFormats(TArray<FName>& OutFormats) const { }

	/**
	 * GetTextureFormatModuleHints is not used and should be deleted
	 */
	virtual void GetTextureFormatModuleHints(TArray<FName>& OutModuleNames) const = 0;

	/**
	 * Platforms that support multiple texture compression variants
	 * might want to use a single specific variant for virtual textures to reduce fragmentation
	 */
	virtual FName FinalizeVirtualTextureLayerFormat(FName Format) const = 0;

	//whether R5G6B5 and B5G5R5A1 is supported
	virtual bool SupportsLQCompressionTextureFormat() const = 0;

	/**
	 * Gets the format to use for a particular piece of audio.
	 *
	 * @param Wave The sound node wave to get the format for.
	 * @return Name of the wave format.
	 */
	virtual FName GetWaveFormat(const class USoundWave* Wave) const = 0;

	/**
	* Gets all the formats which can be returned from GetWaveFormat
	*
	* @param output array of all the formats
	*/
	virtual void GetAllWaveFormats(TArray<FName>& OutFormats) const = 0;
	
	/**
	 * Gets a list of modules that may contain the GetAllWaveFormats. This is optional -
	 * if any required audio format isn't found in this list, then it will use the old path
	 * of loading all audio format modules to gather all available audio formats
	 */
	virtual void GetWaveFormatModuleHints(TArray<FName>& OutModuleNames) const = 0;

	/**
	 * Checks whether if this platform wants AV data (defaults to !IsServerOnly(), which is the standard reason why we don't want AV data)
	 * Used so that custom target platforms can remove AV data, but is not a server-only platform
	 *
	 * @return true if this platform allows AV data to be cooked, false otherwise.
	 */
	virtual bool AllowAudioVisualData() const = 0;

	/** Checks if this Target will want to load this object (generally used to mark an object to not be cooked for this target) */
	virtual bool AllowObject(const class UObject* Object) const = 0;

	/**
	 * Gets the name of the mesh builder module that is responsible for
	 * building static and skeletal mesh derived data for this platform.
	 *
	 * @return The name of a mesh builder module.
	 */
	virtual FName GetMeshBuilderModuleName() const = 0;
#endif

	/**
	 * Package a build for the given platform
	 *
	 * @param InPackgeDirectory The directory that contains what needs to be packaged.
	 * @return bool true on success, false otherwise.
	 */
	virtual bool PackageBuild(const FString& InPackgeDirectory) = 0;

	/**
	 * Returns true if the platform is part of a family of variants
	 */
	virtual bool SupportsVariants() const = 0;

	/**
	 * Gets the variant priority of this platform
	 *
	 * @return float priority for this platform variant.
	 */
	virtual float GetVariantPriority() const = 0;

	/**
	 * Whether or not to send all lower-case filepaths when connecting over a fileserver connection.
	 */
	virtual bool SendLowerCaseFilePaths() const = 0;

	/**
	 * Project settings to check to determine if a build should occur
	 */
	virtual void GetBuildProjectSettingKeys(FString& OutSection, TArray<FString>& InBoolKeys, TArray<FString>& InIntKeys, TArray<FString>& InStringKeys) const = 0;

	/**
	 * Get unique integer identifier for this platform.
	 *
	 * The implementation will assign an ordinal to each target platform at startup, assigning
	 * a value of 0, 1, 2, etc in order to make the ordinals usable as array / bit mask indices.
	 *
	 * @return int32 A unique integer which may be used to identify target platform during the
	 *               current session only (note: not stable across runs).
	 */
	virtual int32 GetPlatformOrdinal() const = 0;

	/**
	 * Given a platform ordinal number, returns the corresponding ITargetPlatform instance
	 */
	TARGETPLATFORM_API static const ITargetPlatformControls* GetPlatformFromOrdinal(int32 Ordinal);

	/**
	 * Returns custom DeviceManager widget creator for this platform
	 */
	virtual TSharedPtr<IDeviceManagerCustomPlatformWidgetCreator> GetCustomWidgetCreator() const = 0;

	/**
	 * Returns wheter or not this 16bit index buffer should be promoted to 32bit
	 */
	virtual bool ShouldExpandTo32Bit(const uint16* Indices, const int32 NumIndices) const = 0;

	/**
	 * Copy a file to the target
	 */
	virtual bool CopyFileToTarget(const FString& DeviceId, const FString& HostFilename, const FString& TargetFilename, const TMap<FString, FString>& CustomPlatformData) = 0;

	/**
	 * Hook called when a TargetPlatform is selected for cooking. Load any required assets or data.
	 * May be called multiple times per process; it is called each time a cook starts for the platform.
	 */
	virtual void InitializeForCook() = 0;

	/**
	 * Gets a list of package names to cook when cooking this platform
	 */
	virtual void GetExtraPackagesToCook(TArray<FName>& PackageNames) const = 0;

	/**
	 * Initializes the host platform to support target devices (may be called multiple times after an SDK is installed while running)
	 */
	virtual bool InitializeHostPlatform() = 0;

	ITargetPlatformSettings* GetTargetPlatformSettings() const
	{
		return TargetPlatformSettings;
	};

	/** Virtual destructor. */
	virtual ~ITargetPlatformControls() = default;

	/**
	 * Gets an event delegate that is executed when a new target device has been discovered.
	 */
	DECLARE_EVENT_OneParam(ITargetPlatformControls, FOnTargetDeviceDiscovered, ITargetDeviceRef /*DiscoveredDevice*/);
	static TARGETPLATFORM_API FOnTargetDeviceDiscovered& OnDeviceDiscovered();

	/**
	 * Gets an event delegate that is executed when a target device has been lost, i.e. disconnected or timed out.
	 */
	DECLARE_EVENT_OneParam(ITargetPlatformControls, FOnTargetDeviceLost, ITargetDeviceRef /*LostDevice*/);
	static TARGETPLATFORM_API FOnTargetDeviceLost& OnDeviceLost();

protected:
	ITargetPlatformSettings* TargetPlatformSettings;
	static TARGETPLATFORM_API int32 AssignPlatformOrdinal(const ITargetPlatformControls& Platform);
};
