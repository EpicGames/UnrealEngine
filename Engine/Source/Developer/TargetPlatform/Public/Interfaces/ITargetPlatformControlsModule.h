// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Interfaces/ITargetPlatformControls.h"

/**
 * Interface for target platform modules.
 */
class ITargetPlatformControlsModule
	: public IModuleInterface
{

public:
	/** Virtual destructor. */
	virtual ~ITargetPlatformControlsModule()
	{
		for (ITargetPlatformControls* TP : AllTargetPlatforms)
		{
			delete TP;
		}
		AllTargetPlatforms.Empty();
	}

	/**
	 * Gets the module's target platforms. This should be overridden by each platform, but
	 * currently, we are re-using the single internal GetTargetPlatform method the old TPModules will implement
	 *
	 * @return The target platform.
	 */
	TArray<ITargetPlatformControls*> GetTargetPlatformControls(FName& PlatformSettingsModuleName)
	{
		if (AllTargetPlatforms.Num() == 0)
		{
			GetTargetPlatformControls(AllTargetPlatforms, PlatformSettingsModuleName);
		}

		return AllTargetPlatforms;
	}

protected:

	/**
	 * This is where each platform module will fill out an array
	*/
	virtual void GetTargetPlatformControls(TArray<ITargetPlatformControls*>& TargetPlatforms, FName& PlatformSettingsModuleName) = 0;

private:
	/** Holds the target platforms. */
	TArray<ITargetPlatformControls*> AllTargetPlatforms;
};
