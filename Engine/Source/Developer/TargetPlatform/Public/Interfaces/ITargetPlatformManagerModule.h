// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Interfaces/ITargetDevice.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class ITargetPlatform;
class ITargetPlatformControls;
class ITargetPlatformSettings;

DECLARE_MULTICAST_DELEGATE(FOnTargetPlatformsInvalidated);

/**
 * Module for the target platform manager
 */
class ITargetPlatformManagerModule
	: public IModuleInterface
{
public:

	static FORCEINLINE ITargetPlatformManagerModule* Get()
	{
		ITargetPlatformManagerModule* Module = FModuleManager::LoadModulePtr<ITargetPlatformManagerModule>("TargetPlatform");
		if (Module)
		{
			Module->Initialize();
		}
		return Module;
	}

	/**
	 * Finds an audio format with the specified name.
	 *
	 * @param Name Name of the format to find.
	 * @return The audio format, or nullptr if not found.
	 */
	virtual const class IAudioFormat* FindAudioFormat( FName Name ) = 0;

	/**
	 * Finds a PhysX format with the specified name.
	 *
	 * @param Name Name of the format to find.
	 * @return The PhysX format, or nullptr if not found.
	 */
	virtual const class IPhysXCooking* FindPhysXCooking( FName Name ) = 0;

	/**
	 * Finds a shader format with the specified name.
	 *
	 * @param Name Name of the format to find.
	 * @return The shader format, or nullptr if not found.
	 */
	virtual const class IShaderFormat* FindShaderFormat( FName Name ) = 0;

	/**
	 * Finds a texture format with the specified name.
	 *
	 * @param Name Name of the format to find.
	 * @return The texture format, or nullptr if not found.
	 */
	virtual const class ITextureFormat* FindTextureFormat( FName Name ) = 0;

	/**
	 * Finds a target device by identifier.
	 *
	 * @param DeviceId The device identifier.
	 * @return The target device, or nullptr if not found.
	 */
	virtual ITargetDevicePtr FindTargetDevice( const FTargetDeviceId& DeviceId ) = 0;

	/**
	 * Finds a target platform by name.
	 *
	 * @param Name The target platform's short or long name.
	 * @return The target platform, or nullptr if not found.
	 */
	virtual ITargetPlatform* FindTargetPlatform( FStringView Name ) = 0;

	/**
	 * Finds a target platform by name.
	 *
	 * @param Name The target platform's short or long name.
	 * @return The target platform, or nullptr if not found.
	 */
	virtual ITargetPlatform* FindTargetPlatform(FName Name) = 0;

	/**
	 * Finds a target platform by name.
	 *
	 * @param Name The target platform's short or long name.
	 * @return The target platform, or nullptr if not found.
	 */
	virtual ITargetPlatform* FindTargetPlatform(const TCHAR* Name) = 0;

	/**
	 * Finds a target platform by looking for one that supports a given value for a generic type of support
	 *
	 * @param Name SupportClass The type of support needed (like "ShaderFormat")
	 * @param RequiredSupportValue The value of the supported type that is needed
	 * @return The target platform, or nullptr if not found.
	 */
	virtual ITargetPlatform* FindTargetPlatformWithSupport(FName SupportType, FName RequiredSupportedValue) = 0;

	/**
	 * Return the list of platforms which we need to support when cooking (only set when actually cooking)
	 *
	 * @return Collection of platforms.
	 */
	UE_DEPRECATED(5.1, "Use GetActiveTargetPlatforms instead of GetCookingTargetPlatforms")
	virtual const TArray<ITargetPlatform*>& GetCookingTargetPlatforms() = 0;

	/**
	 * Return the list of the ITargetPlatforms that we want to build data for.
	 *
	 * @return Collection of platforms.
	 */
	virtual const TArray<ITargetPlatform*>& GetActiveTargetPlatforms() = 0;

	/**
	 * Returns the list of all IAudioFormats that were located in DLLs.
	 *
	 * @return Collection of audio formats.
	 */
	virtual const TArray<const class IAudioFormat*>& GetAudioFormats() = 0;

	/**
	 * Returns the list of all IPhysXCooking that were located in DLLs.
	 *
	 * @return Collection of PhysX formats.
	 */
	virtual const TArray<const class IPhysXCooking*>& GetPhysXCooking() = 0;

	/**
	 * Returns the target platform that is currently running.
	 *
	 * Note: This method only returns a Target Platform when WITH_EDITOR.
	 *
	 * @return Running target platform.
	 */
	virtual ITargetPlatform* GetRunningTargetPlatform() = 0;

	/**
	 * Returns the list of all ITextureFormats that were located in DLLs.
	 *
	 * @return Collection of shader formats.
	 */
	virtual const TArray<const class IShaderFormat*>& GetShaderFormats() = 0;

	/**
	 * Returns the list of all ITargetPlatforms that were located in DLLs.
	 *
	 * @return Collection of platforms.
	 */
	virtual const TArray<ITargetPlatform*>& GetTargetPlatforms() = 0;
	virtual const TArray<ITargetPlatformControls*>& GetTargetPlatformControls() = 0;
	virtual const TArray<ITargetPlatformSettings*>& GetTargetPlatformSettings() = 0;

	/**
	 * Returns the list of all ITextureFormats that were located in DLLs.
	 *
	 * @return Collection of texture formats.
	 */
	UE_DEPRECATED(5.6, "Not thread safe, use FindTextureFormat")
	TArray<const class ITextureFormat*> GetTextureFormats() { return {}; }

	/**
	 * Determine if there were errors during the initialization of the platform manager.
	 *
	 * @param OutErrorMessages Optional pointer to an FString that will have the error messages appended to it.
	 * @return True if there were errors during the initialization of the platform manager, False otherwise.
	 */
	virtual bool HasInitErrors(FString* OutErrorMessages) const = 0;
	
	/**
	 * Invalidates the target platform module.
	 *
	 * Invalidate should be called if any TargetPlatform modules get loaded/unloaded/reloaded during 
	 * runtime to give the implementation the chance to rebuild all its internal states and caches.
	 */
	virtual void Invalidate() = 0;

	/**
	 * Checks whether we should only build formats that are actually required for use by the runtime.
	 *
	 * @return true if formats are restricted, false otherwise.
	 */
	virtual bool RestrictFormatsToRuntimeOnly() = 0;

	/**
	 * Gets the shader format version for the specified shader.
	 *
	 * @param Name Name of the shader format to get the version for.
	 * @return Version number.
	 */
	virtual uint32 ShaderFormatVersion(FName Name) = 0;

	/**
	 * Allows changes to environment for a given platform
	 */
	UE_DEPRECATED(5.8, "Deprecated, please use SetupEnvironmentVariables instead")
	virtual bool UpdatePlatformEnvironment(const FString& PlatformName, TArray<FString> &Keys, TArray<FString> &Values) final { return false;};

	/**
	 * Allows changes to the environment
	 */
	virtual void SetupEnvironmentVariables(TArray<FString>& EnvVarNames, const TArray<FString>& EnvVarValues) = 0;

	/** A callback that holders of ITargetPlatform* must subscribe to to be notified of when the ITargetPlatform* has been invalidated and should be requeried from e.g. FindTargetPlatform */
	virtual FOnTargetPlatformsInvalidated& GetOnTargetPlatformsInvalidatedDelegate() = 0;
	/**
	 * After installing an SDK with Turnkey, this will refresh the TargetPlatform, find devices, etc
	 */
	virtual bool UpdateAfterSDKInstall(FName TargetPlatformName) = 0;


	/** Virtual destructor. */
	~ITargetPlatformManagerModule() = default;

private:
	/**
	 * Performs all module setup (SDK validation, platform discovery, format loading), called after the module is 
	 * loaded and before any other methods are used.
	 * Separating this from construction allows the async UBT process (kicked off via FDelayedAutoRegisterHelper)
	 * more time to finish before we block waiting for it.
	 */
	virtual void Initialize() = 0;
};
