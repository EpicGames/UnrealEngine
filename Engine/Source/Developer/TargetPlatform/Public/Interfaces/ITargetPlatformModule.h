// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformSettings.h"


class ITargetPlatformControlsModule;
class ITargetPlatformSettingsModule;
class ITargetPlatformControls;
/**
 * Interface for target platform modules.
 */
class ITargetPlatformModule
	: public IModuleInterface
{

public:

	/** Virtual destructor. */
	virtual ~ITargetPlatformModule()
	{
		for (ITargetPlatform* TP : AllTargetPlatforms)
		{
			delete TP;
		}
		AllTargetPlatforms.Empty();
	}

	/**
	 * Gets the module's target platforms. This should be overridden by each platform, but 
	 * currently, we are re-using the single internal GetTargetPlatform method the old TPModules will implement
	 *
	 * @return The target platform.
	 */
	TArray<ITargetPlatform*> GetTargetPlatforms()
	{
		if (AllTargetPlatforms.Num() == 0)
		{
			GetTargetPlatforms(AllTargetPlatforms, PlatformSettings, PlatformControls);
		}

		return AllTargetPlatforms;
	}

	TArray<ITargetPlatformSettings*> PlatformSettings;
	TArray<ITargetPlatformControls*> PlatformControls;

protected:

	/**
	 * This is where each platform module will fill out an array
	*/
	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms) = 0;
	virtual void GetTargetPlatforms(TArray<ITargetPlatform*>& TargetPlatforms, TArray<ITargetPlatformSettings*> TargetPlatformSettings, TArray<ITargetPlatformControls*> TargetPlatformControls)
	{
		GetTargetPlatforms(TargetPlatforms);
	}
private:
	/** Holds the target platforms. */
	TArray<ITargetPlatform*> AllTargetPlatforms;
};
