// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "PlatformInfo.h"

enum class EOfflineBVHMode
{
	Disabled,
	MaximizePerformance,
	MinimizeMemory,
};

enum class ERayTracingRuntimeMode : uint8
{
	Disabled,
	Inline,
	Full
};

/**
 * Enumerates features that may be supported by target platforms.
 */
enum class ETargetPlatformFeatures
{
	/** Audio Streaming */
	AudioStreaming,

	/** Distance field shadows. */
	DistanceFieldShadows,

	/** Distance field AO. */
	DistanceFieldAO,

	/** Gray scale SRGB texture formats support. */
	GrayscaleSRGB,

	/** High quality light maps. */
	HighQualityLightmaps,

	/** Low quality light maps. */
	LowQualityLightmaps,

	/** Run multiple game instances on a single device. */
	MultipleGameInstances,

	/** Builds can be packaged for this platform. */
	Packaging,

	/** Connect and disconnect devices through the SDK. */
	SdkConnectDisconnect,

	/** Texture streaming. */
	TextureStreaming,

	/** Mesh LOD streaming. */
	MeshLODStreaming,

	/** Landscape visual mesh LOD streaming. */
	LandscapeMeshLODStreaming UE_DEPRECATED(5.1, "LandscapeMeshLODStreaming is now deprecated and will be removed."),

	/** User credentials are required to use the device. */
	UserCredentials,

	/** The platform uses the mobile forward pipeline */
	MobileRendering,

	/** The platform uses the deferred pipeline, typically PC/Console platforms */
	DeferredRendering,

	/* Should split paks into smaller sized paks */
	ShouldSplitPaksIntoSmallerSizes,

	/* The platform supports half float vertex format */
	HalfFloatVertexFormat,

	/* The platform supports the experimental Device Output Log window */
	DeviceOutputLog,

	/* The platform supports memory mapped files */
	MemoryMappedFiles,

	/* The platform supports memory mapped audio */
	MemoryMappedAudio,

	/* The platform supports memory mapped animation */
	MemoryMappedAnimation,

	/* The platform supports sparse textures */
	SparseTextures,

	/* Can we use the virtual texture streaming system on this platform. */
	VirtualTextureStreaming UE_DEPRECATED(5.6, "VirtualTextureStreaming is now deprecated and will be removed."),

	/** Lumen Global Illumination. */
	LumenGI,

	/** The platform supports hardware LZ decompression */
	HardwareLZDecompression,

	/* The platform makes use of extra cook-time file region metadata in its packaging process. */
	CookFileRegionMetadata,

	/** The platform supports communication (reading and writing data) between a target a connected PC. */
	DirectDataExchange,

	/** The platform supports Luminance + Alpha encoding mode for normalmaps */
	NormalmapLAEncodingMode,

	/** Should the Aggregate Device be shown for this platform (ex: All_Android_on_...) */
	ShowAggregateDevice,

	/** Does the platform allow various connection types to be used (ie: wifi and usb) */
	SupportsMultipleConnectionTypes,
	
	/** The platform can cook packages (e.g. CookedCooker) */
	CanCookPackages
};

class ITargetPlatformSettings
{
public:
	/**
	 * Returns the config system object usable by this TargetPlatform. It should not be modified in anyway
	 */
	virtual FConfigCacheIni* GetConfigSystem() const = 0;

	/**
	 * Gets the platform's INI name (so an offline tool can load the INI for the given target platform).
	 *
	 * @see PlatformName
	 */
	virtual FString IniPlatformName() const = 0;

	/**
	 * Checks whether the target platform supports the specified value for the specified type of support
	 *
	 * @param SupportedType The type of support being queried
	 * @param RequiredSupportedValue The value of support needed
	 * @return true if the feature is supported, false otherwise.
	 */
	virtual bool SupportsValueForType(FName SupportedType, FName RequiredSupportedValue) const = 0;

	/**
	 * Gets whether the platform should use forward shading or not.
	 */
	virtual bool UsesForwardShading() const = 0;

	/**
	* Gets whether the platform should use DBuffer for decals.
	*/
	virtual bool UsesDBuffer() const = 0;

	/**
	* Gets whether the platform should output velocity in the base pass.
	*/
	virtual bool UsesBasePassVelocity() const = 0;

	/**
	* Gets whether the platform will use selective outputs in the base pass shaders.
	*/
	virtual bool UsesSelectiveBasePassOutputs() const = 0;

	/**
	* Gets whether Nanite fallback meshes should be stripped from cooked builds for the platform.
	*/
	virtual bool ShouldStripNaniteFallbackMeshes() const = 0;

	/**
	* Gets whether the platform will use distance fields.
	*/
	virtual bool UsesDistanceFields() const = 0;

	/**
	* Gets whether the platform will use ray tracing.
	*/
	virtual bool UsesRayTracing() const = 0;

	/**
	* Gets ray tracing runtime mode (disabled/inline/hit-lighting).
	*/
	virtual ERayTracingRuntimeMode GetRayTracingMode() const = 0;

	/**
	 * Gets a platform-dependent bitfield describing which hardware generations are supported.
	 * Applies to platforms with multiple iterations of the device, which may vary in performance.
	 * Each bit represents a supported device version.
	 * Return 0 if information is not provided by the platform.
	 */
	virtual uint32 GetSupportedHardwareMask() const = 0;

	/**
	* Gets static mesh offline BVH mode
	*/
	virtual EOfflineBVHMode GetStaticMeshOfflineBVHMode() const = 0;

	/**
	* Gets whether the platform will use compression for static mesh offline BVH.
	*/
	virtual bool GetStaticMeshOfflineBVHCompression() const = 0;

	/**
	* Gets skeletal mesh offline BVH mode
	*/
	virtual EOfflineBVHMode GetSkeletalMeshOfflineBVHMode() const = 0;

	/**
	* Gets whether the platform will use SH2 instead of SH3 for sky irradiance.
	*/
	virtual bool ForcesSimpleSkyDiffuse() const = 0;

	/**
	* Gets whether the platform will encode depth velocity.
	*/
	virtual bool VelocityEncodeDepth() const = 0;

	/**
	* Gets down sample mesh distance field divider.
	*
	* @return 1 if platform does not need to downsample mesh distance fields
	*/
	virtual float GetDownSampleMeshDistanceFieldDivider() const = 0;

	/**
	* Gets an integer representing the height fog mode for opaque materials on a platform.
	* @return 0 if no override (i.e. use r.VertexFoggingForOpaque from project settings); 1 if pixel fog; 2 if vertex fog.
	*/
	virtual int32 GetHeightFogModeForOpaque() const = 0;

	/**
	 * Gets whether the platform should use DBuffer for decals when using the mobile renderer.
	 */
	virtual bool UsesMobileDBuffer() const = 0;

	/**
	 * Gets whether the platform uses ASTC HDR
	 */
	virtual bool UsesASTCHDR() const = 0;

	/**
	 * Gets the fallback format when ASTC HDR is not supported
	 */
	virtual FName GetFallbackASTCHDR() const = 0;

	/**
	* Gets the shader formats this platform can use.
	*
	* @param OutFormats Will contain the shader formats.
	*/
	virtual void GetAllPossibleShaderFormats(TArray<FName>& OutFormats) const = 0;

	/**
	* Gets the shader formats that have been selected for this target platform
	*
	* @param OutFormats Will contain the shader formats.
	*/
	virtual void GetAllTargetedShaderFormats(TArray<FName>& OutFormats) const = 0;

	/**
	* Gets the shader formats that support ray tracing for this target platform.
	*
	* @param OutFormats Will contain the shader formats.
	*/
	virtual void GetRayTracingShaderFormats(TArray<FName>& OutFormats) const = 0;

	/**
	 * Checks whether the target platform supports the specified feature.
	 *
	 * @param Feature The feature to check.
	 * @return true if the feature is supported, false otherwise.
	 */
	virtual bool SupportsFeature(ETargetPlatformFeatures Feature) const = 0;

	/**
	 * Get the possible architectures for the target platform
	 *
	 * @param OutArchitectures will contain supported architectures.
	 */
	virtual void GetPossibleArchitectures(TArray<FString>& OutArchitectures) const = 0;

	/**
	 * Get the host architectures for the target platform, if any
	 *
	 * @return host architecture, or an empty string if it is not relevant
	 */
	virtual FString GetHostArchitecture() const = 0;

#if WITH_ENGINE
	/**
	 * Gets a list of modules that may contain the GetAllTargetedShaderFormats. This is optional -
	 * if any required shader format isn't found in this list, then it will use the old path
	 * of loading all shader format modules to gather all available shader formats
	 */
	virtual void GetShaderFormatModuleHints(TArray<FName>& OutModuleNames) const = 0;

	/**
	 * Gets the reflection capture formats this platform needs.
	 *
	 * @param OutFormats Will contain the collection of formats.
	 */
	virtual void GetReflectionCaptureFormats(TArray<FName>& OutFormats) const = 0;

	/**
	 * Gets the texture LOD settings used by this platform.
	 *
	 * @return A texture LOD settings structure.
	 *
	 * This should generally be equal to :
	 *	const UTextureLODSettings* LODSettings = UDeviceProfileManager::Get().FindProfile(TargetPlatform->CookingDeviceProfileName())->GetTextureLODSettings();
	 * It is cached by RegisterTextureLODSettings()
	 *
	 */
	virtual const class UTextureLODSettings& GetTextureLODSettings() const = 0;

	/**
	* Register Basic LOD Settings for this platform
	*/
	virtual void RegisterTextureLODSettings(const class UTextureLODSettings* InTextureLODSettings) = 0;

	/**
	 * Gets the static mesh LOD settings used by this platform.
	 *
	 * @return A static mesh LOD settings structure.
	 */
	virtual const class FStaticMeshLODSettings& GetStaticMeshLODSettings() const = 0;
#endif

	/**
	 * Checks whether this platform is the platform that's currently running.
	 *
	 * For example, when running on Windows, the Windows ITargetPlatform will return true
	 * and all other platforms will return false.
	 *
	 * @return true if this platform is running, false otherwise.
	 */
	virtual bool IsRunningPlatform() const = 0;

	/** Virtual destructor. */
	virtual ~ITargetPlatformSettings() = default;
};
