// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/**
 * Implements a target devices identifier.
 */
class FTargetDeviceId
{
public:

	/** Default constructor. */
	FTargetDeviceId() { }

	/**
	 * Creates and initializes a new target device identifier.
	 *
	 * @param InPlatformName The name of the device's target platform.
	 * @param InDeviceName The unique device name, i.e. IP address or computer name.
	 */
	FTargetDeviceId( const FString& InPlatformName, const FString& InDeviceName )
		: PlatformName(InPlatformName)
		, DeviceName(InDeviceName)
	{ }

public:

	/**
	 * Compares this device identifier with the given device identifier for equality.
	 *
	 * @param Other The device identifier to compare with.
	 * @return true if the identifiers are equal, false otherwise.
	 */
	bool operator==( const FTargetDeviceId& Other ) const
	{
		return ((PlatformName == Other.PlatformName) && (DeviceName == Other.DeviceName));
	}

	/**
	 * Compares this device identifier with the given device identifier for inequality.
	 *
	 * @param Other The device identifier to compare with.
	 * @return true if the identifiers are not equal, false otherwise.
	 */
	bool operator!=( const FTargetDeviceId& Other ) const
	{
		return ((PlatformName != Other.PlatformName) || (DeviceName != Other.DeviceName));
	}

	/**
	 * Serializes the given device identifier from or into the specified archive.
	 *
	 * @param Ar The archive to serialize from or into.
	 * @param Uri The device identifier to serialize.
	 * @return The archive.
	 */
	friend FArchive& operator<<( FArchive& Ar, FTargetDeviceId& Uri )
	{
		return Ar << Uri.PlatformName << Uri.DeviceName;
	}

public:

	/**
	 * Gets the device identifier.
	 *
	 * @return Device identifier.
	 */
	const FString& GetDeviceName() const
	{
		return DeviceName;
	}

	/**
	 * Gets the platform name.
	 *
	 * @return Platform name.
	 */
	const FString& GetPlatformName() const
	{
		return PlatformName;
	}

	/**
	 * Checks whether this target device identifier is valid.
	 *
	 * @return true if the identifier is valid, false otherwise.
	 */
	bool IsValid() const
	{
		return (!PlatformName.IsEmpty() && !DeviceName.IsEmpty());
	}

	/**
	 * Gets the string representation for this device identifier.
	 *
	 * @return String representation.
	 */
	FString ToString() const
	{
		return PlatformName + TEXT("@") + DeviceName;
	}

public:

	/**
	 * Gets the hash for specified target device identifier.
	 *
	 * @param Uri The device identifier to get the hash for.
	 * @return Hash value.
	 */
	friend uint32 GetTypeHash( const FTargetDeviceId& Uri )
	{
		return GetTypeHash(Uri.ToString());
	}

public:

	/**
	 * Converts a string to a target device identifier.
	 *
	 * @param IdString The string to convert.
	 * @param OutId Will contain the parsed device identifier.
	 * @return true if the string was converted successfully, false otherwise.
	 */
	static bool Parse( const FString& IdString, FTargetDeviceId& OutId )
	{
		return IdString.Split(TEXT("@"), &OutId.PlatformName, &OutId.DeviceName, ESearchCase::CaseSensitive);
	}

private:

	/** Holds the name of the device's target platform. */
	FString PlatformName;

	/** Holds the name of the target device. */
	FString DeviceName;
};
