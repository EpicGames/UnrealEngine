// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once


/* Boilerplate
 *****************************************************************************/

#include "Misc/MonolithicHeaderBoilerplate.h"
MONOLITHIC_HEADER_BOILERPLATE()


/* Public dependencies
 *****************************************************************************/

#include "Modules/ModuleManager.h"


/* Public interfaces
 *****************************************************************************/

#include "Interfaces/TargetDeviceId.h"

#include "Interfaces/ITargetDevice.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformModule.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/ITargetDeviceOutput.h"

#include "Interfaces/IAudioFormat.h"
#include "Interfaces/IAudioFormatModule.h"

#include "Interfaces/IShaderFormat.h"
#include "Interfaces/IShaderFormatModule.h"

#include "Interfaces/ITextureFormat.h"
#include "Interfaces/ITextureFormatModule.h"
