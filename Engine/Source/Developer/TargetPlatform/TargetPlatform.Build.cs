// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class TargetPlatform : ModuleRules
{
	public TargetPlatform(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("SlateCore");
		PrivateDependencyModuleNames.Add("Slate");
		PrivateDependencyModuleNames.Add("Projects");
		PrivateDependencyModuleNames.Add("RenderCore");
		PrivateDependencyModuleNames.Add("Json");

		PublicDependencyModuleNames.Add("DeveloperSettings");
		PublicDependencyModuleNames.Add("AudioPlatformConfiguration");
		PublicDependencyModuleNames.Add("DesktopPlatform");
		PublicDependencyModuleNames.Add("Analytics");
		PublicDependencyModuleNames.Add("RHI");

		// TextureFormat contains public headers that were historically part of TargetPlatform, so it is exposed
		// as a public include path on TargetPlatform.
		PublicIncludePathModuleNames.Add("TextureFormat");
		PublicDependencyModuleNames.Add("TextureFormat");

		PrivateIncludePathModuleNames.Add("Engine");
		PrivateIncludePathModuleNames.Add("PhysicsCore");

		if (Target.bCompileAgainstEngine)
		{
			PrivateDependencyModuleNames.Add("Engine");
		}

		DynamicallyLoadedModuleNames.Add("TurnkeySupport");
		PrivateIncludePathModuleNames.Add("TurnkeySupport");

		// no need for all these modules if the program doesn't want developer tools at all (like UnrealFileServer)
		if (!Target.bBuildRequiresCookedData && Target.bBuildDeveloperTools)
		{
			// these are needed by multiple platform specific target platforms, so we make sure they are built with the base editor
			DynamicallyLoadedModuleNames.Add("ShaderPreprocessor");
			DynamicallyLoadedModuleNames.Add("ShaderFormatOpenGL");
            DynamicallyLoadedModuleNames.Add("ShaderFormatVectorVM");
            DynamicallyLoadedModuleNames.Add("ImageWrapper");

			// @todo: should move all of this to specific UEBuild*.cs files
			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				// these are needed by multiple platform specific target platforms, so we make sure they are built with the base editor
				DynamicallyLoadedModuleNames.Add("ShaderFormatD3D");

				DynamicallyLoadedModuleNames.Add("TextureFormatOodle");
				DynamicallyLoadedModuleNames.Add("TextureFormatUncompressed");

				if (Target.bCompileAgainstEngine)
				{
					DynamicallyLoadedModuleNames.Add("AudioFormatADPCM");
					DynamicallyLoadedModuleNames.Add("AudioFormatOgg");
					DynamicallyLoadedModuleNames.Add("AudioFormatOpus");
					DynamicallyLoadedModuleNames.Add("AudioFormatBink");
					DynamicallyLoadedModuleNames.Add("AudioFormatRad");
				}
			}
			else if (Target.Platform == UnrealTargetPlatform.Mac)
			{
				DynamicallyLoadedModuleNames.Add("TextureFormatOodle");
				DynamicallyLoadedModuleNames.Add("TextureFormatUncompressed");

				if (Target.bCompileAgainstEngine)
				{
					DynamicallyLoadedModuleNames.Add("AudioFormatADPCM");
					DynamicallyLoadedModuleNames.Add("AudioFormatOgg");
					DynamicallyLoadedModuleNames.Add("AudioFormatOpus");
					DynamicallyLoadedModuleNames.Add("AudioFormatBink");
					DynamicallyLoadedModuleNames.Add("AudioFormatRad");
				}

			}
			else if (Target.IsInPlatformGroup(UnrealPlatformGroup.Linux))
			{
				DynamicallyLoadedModuleNames.Add("TextureFormatOodle");
				DynamicallyLoadedModuleNames.Add("TextureFormatUncompressed");

				if (Target.bCompileAgainstEngine)
				{
					DynamicallyLoadedModuleNames.Add("AudioFormatADPCM");
					DynamicallyLoadedModuleNames.Add("AudioFormatOgg");
					DynamicallyLoadedModuleNames.Add("AudioFormatOpus");
					DynamicallyLoadedModuleNames.Add("AudioFormatBink");
					DynamicallyLoadedModuleNames.Add("AudioFormatRad");
				}
			}
		}
	}
}
